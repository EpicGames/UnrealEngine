<?xml version="1.0" encoding="UTF-8"?>
<!DOCTYPE plist PUBLIC "-//Apple//DTD PLIST 1.0//EN" "http://www.apple.com/DTDs/PropertyList-1.0.dtd">
<plist version="1.0">
<dict>
	<key>CFBundleURLTypes</key>
	<array>
		<dict>
			<key>CFBundleURLName</key>
			<string>com.Epic.Unreal</string>
			<key>CFBundleURLSchemes</key>
			<array>
				<string>${EXECUTABLE_NAME}</string>
			</array>
		</dict>
	</array>
	<key>CFBundleDevelopmentRegion</key>
	<string>English</string>
	<key>CFBundleDisplayName</key>
	<string>Mobile Tool</string>
	<key>CFBundleExecutable</key>
	<string>${EXECUTABLE_NAME}</string>
	<key>CFBundleIdentifier</key>
	<string>com.epicgames.UnrealMobileTool</string>
	<key>CFBundleInfoDictionaryVersion</key>
	<string>6.0</string>
	<key>CFBundleName</key>
	<string>${EXECUTABLE_NAME}</string>
	<key>CFBundlePackageType</key>
	<string>APPL</string>
	<key>CFBundleSignature</key>
	<string>????</string>
	<key>CFBundleVersion</key>
	<string>10246.0</string>
	<key>LSRequiresIPhoneOS</key>
	<true/>
	<key>UIStatusBarHidden</key>
	<true/>
	<key>UIInterfaceOrientation</key>
	<string>UIInterfaceOrientationLandscapeRight</string>
	<key>UISupportedInterfaceOrientations</key>
	<array>
		<string>UIInterfaceOrientationLandscapeRight</string>
		<string>UIInterfaceOrientationLandscapeLeft</string>
	</array>
	<key>UIRequiredDeviceCapabilities</key>
	<array>
		<string>armv7</string>
		<string>opengles-2</string>
	</array>
  <key>CFBundleIcons</key>
  <dict>
    <key>CFBundlePrimaryIcon</key>
    <dict>
	    <key>CFBundleIconFiles</key>
	    <array>
		    <string>Icon29.png</string>
		    <string>Icon40.png</string>
		    <string>Icon57.png</string>
		    <string>Icon60.png</string>
	    </array>
    </dict>
  </dict>
  <key>CFBundleIcons~ipad</key>
  <dict>
    <key>CFBundlePrimaryIcon</key>
    <dict>
	    <key>CFBundleIconFiles</key>
	    <array>
		    <string>Icon29.png</string>
		    <string>Icon40.png</string>
		    <string>Icon50.png</string>
		    <string>Icon72.png</string>
		    <string>Icon76.png</string>
	    </array>
    </dict>
  </dict>
	<key>CFBundleSupportedPlatforms</key>
	<array>
		<string>iPhoneOS</string>
	</array>
	<key>UIPrerenderedIcon</key>
	<true/>
</dict>
</plist>
