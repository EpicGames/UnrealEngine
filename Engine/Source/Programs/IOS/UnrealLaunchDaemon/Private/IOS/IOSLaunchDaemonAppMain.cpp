//
//  main.m
//  UnrealLaunchDaemon
//
//  Created by Russell Nelson on 11/26/12.
//  Copyright (c) 2012 zombie. All rights reserved.
//

#import <UIKit/UIKit.h>

#import "IOSAppDelegate.h"
#include "IOS/IOSCommandLineHelper.h"
#include "IOSLaunchDaemonView.h"

#define IOS_MAX_PATH 1024
#define CMD_LINE_MAX 16384

FString GSavedCommandLine;
FLaunchDaemonMessageHandler GCommandSystem;

void FAppEntry::Suspend()
{
}

void FAppEntry::Resume()
{
}

void FAppEntry::SuspendTick()
{
}

bool FAppEntry::IsStartupMoviePlaying()
{
	return false;
}

void FAppEntry::PreInit(IOSAppDelegate* AppDelegate, UIApplication* Application)
{
	Application.idleTimerDisabled = YES;

    // Setup a view controller.
    UIViewController* pController = [[IOSLaunchDaemonViewController alloc] init];

    // Setup View.

    AppDelegate.RootView = [pController view];

	[AppDelegate.Window setRootViewController:pController];
}

void FAppEntry::PlatformInit()
{
}

void FAppEntry::Init()
{
	bool DirectLaunch = false;
	FString CmdLine(FCommandLine::Get());
	NSLog(@"ULD cmdline %s", (TCHAR_TO_ANSI(*CmdLine)));
	int32 Index = CmdLine.Find("-directLaunch ");
	FString RealCmdLine;
	if ( Index > -1 )
	{
		// We got a request for a direct launch. This means the game triggered the launch of ULD because it was running when UFE requested a launch.
		// Wait 2 seconds and then relaunch the game

		NSLog(@"Found direct Launch\n");
		RealCmdLine = CmdLine.Mid(Index + 14); // "-directLaunch " is 14 characters. 
		NSLog(@"Stripped cmd line: %s", TCHAR_TO_ANSI(*RealCmdLine));

		// Clear the args so that this app (which is an actual Unreal app) doesn't try to communicate with the cookonthefly server, etc. ;-)
		FIOSCommandLineHelper::InitCommandArgs(TEXT(""));

		DirectLaunch = true;
	}

	GEngineLoop.PreInit(FCommandLine::Get());

	if ( DirectLaunch )
	{
		// We got a request for a direct launch. This means the game triggered the launch of ULD because it was running when UFE requested a launch.
		// Wait 2 seconds and then relaunch the game

		double WaitStartTime = FPlatformTime::Seconds();

		NSLog(@"Found direct launch command, waiting 2 seconds");

		while ( WaitStartTime + 2.0 > FPlatformTime::Seconds() )
		{
			FPlatformProcess::Sleep(0);
		}

		NSLog(@"ULD direct launching command: %s", TCHAR_TO_ANSI(*RealCmdLine));
		GCommandSystem.Launch(RealCmdLine);

	}

	// initialize task graph
	FTaskGraphInterface::Startup(FPlatformMisc::NumberOfWorkerThreadsToSpawn());
	FTaskGraphInterface::Get().AttachToThread(ENamedThreads::GameThread);

	// initialize messaging subsystem
	FModuleManager::LoadModuleChecked<IMessagingModule>("Messaging");

	// load the messaging plugin
	FModuleManager::Get().LoadModule("UdpMessaging");

	//Set up the message handling to interface with other endpoints on our end.
	NSLog(@"%s", "Initializing Communications in ULD mode\n");
	GCommandSystem.Init();
}

void FAppEntry::Tick()
{
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

	FPlatformProcess::Sleep(0);
}

void FAppEntry::Shutdown()
{
	NSLog(@"%s", "ULD: Shutting down Communications\n");
	GCommandSystem.Shutdown();
	FTaskGraphInterface::Shutdown();
}

int main(int argc, char *argv[])
{
	for (int32 Option = 1; Option < argc; Option++)
	{
		GSavedCommandLine += TEXT(" ");
		GSavedCommandLine += ANSI_TO_TCHAR(argv[Option]);
	}

	FIOSCommandLineHelper::InitCommandArgs(TEXT("-messaging"));

    @autoreleasepool {
        return UIApplicationMain(argc, argv, nil, NSStringFromClass([IOSAppDelegate class]));
    }
}
