// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSLaunchDaemonView.h"
#include "IOS/IOSAppDelegate.h"

#import <ifaddrs.h>
#import <arpa/inet.h>

@implementation NSString (StringSizeWithFont)

- (CGSize) sizeWithFontSafe:(UIFont *)fontToUse
{
    if ([self respondsToSelector:@selector(sizeWithAttributes:)])
    {
        NSDictionary* attribs = @{NSFontAttributeName:fontToUse};
        return ([self sizeWithAttributes:attribs]);
    }
    return ([self sizeWithFont:fontToUse]);
}

@end

@implementation IOSLaunchDaemonView

// Get IP Address
- (NSString *)getIPAddress
{    
    NSString *address = @"Invalid";
    struct ifaddrs *interfaces = NULL;
    struct ifaddrs *temp_addr = NULL;
    int success = 0;
    success = getifaddrs(&interfaces);
    if (success == 0) 
    {
        temp_addr = interfaces;
        while(temp_addr != NULL) 
        {
            if(temp_addr->ifa_addr->sa_family == AF_INET) 
            {
                if([[NSString stringWithUTF8String:temp_addr->ifa_name] isEqualToString:@"en0"])
                {
                    address = [NSString stringWithUTF8String:inet_ntoa(((struct sockaddr_in *)temp_addr->ifa_addr)->sin_addr)];               
                }
            }
            temp_addr = temp_addr->ifa_next;
        }
    }
    freeifaddrs(interfaces);
    return address;
} 

- (id)initWithFrame:(CGRect)Frame
{
	if ((self = [super initWithFrame:Frame]))
	{
	    // Setup Spinner.
        pSpinner = [[UIActivityIndicatorView alloc] initWithActivityIndicatorStyle: UIActivityIndicatorViewStyleWhiteLarge];
        [pSpinner setCenter:[self center]];

        [self addSubview:pSpinner];
        [pSpinner release];
        [pSpinner startAnimating];

        // Obtain our previous URL.
        NSUserDefaults *defaults = [NSUserDefaults standardUserDefaults];
        NSString *previousLaunchURL = [defaults objectForKey:@"PreviousLaunchURL"];

        // Display previous URL and our IP Address.
        UIFont* largeFont = [UIFont fontWithName:@"Arial Rounded MT Bold" size:(24.0)];
        UIFont* smallFont = [UIFont fontWithName:@"Arial Rounded MT Bold" size:(18.0)];

        NSString* urlTitle = (previousLaunchURL == nil) ? @"No Previous Launch" : @"Previous Launch:";

        const float kBuffer = 16.0f;

        float xOffset = kBuffer;  // Start with a bit of a buffer.
        float yOffset = kBuffer;

        CGSize labelSize = [urlTitle sizeWithFontSafe:largeFont];
        UILabel *label   = [[UILabel alloc] initWithFrame:CGRectMake( xOffset, 
                                                                      yOffset, 
                                                                      labelSize.width, 
                                                                      labelSize.height ) ];
        label.textAlignment   =  NSTextAlignmentCenter;
        label.textColor       = [UIColor whiteColor];
        label.backgroundColor = [UIColor blackColor];
        label.numberOfLines   = 0;
        label.font            = largeFont;
        label.text            = urlTitle;
        [self addSubview:label];

        yOffset += labelSize.height + kBuffer;

        if( previousLaunchURL != nil )
        {
            // We want to decompose the launch url so that we can re-create it with
            // only the arguments that make sense...

            NSArray* IgnoreArgs = [NSArray arrayWithObjects: @"InstanceId",
                                                             @"SessionId",
                                                             @"nomcp",
                                                             @"stdout",
                                                             @"Messaging",
                                                             nil ];


            NSArray*  splitURL    = [previousLaunchURL componentsSeparatedByString:@"-"];
            NSString* rebuiltURL  = @"";

            for(NSString* comp in splitURL)
            {
                bool bIgnore = false;
                for(NSString* ignore in IgnoreArgs )
                {
                    if( [comp hasPrefix: ignore] )
                    {
                        bIgnore = true;
                        break;
                    }
                }
                if( bIgnore )
                {
                    continue;
                }

                rebuiltURL = [rebuiltURL stringByAppendingFormat:@"%@\n", comp];
            }

            labelSize = [rebuiltURL sizeWithFontSafe:smallFont];
            label = [[UILabel alloc] initWithFrame:CGRectMake( xOffset, yOffset, labelSize.width, labelSize.height ) ];
            label.textAlignment =  NSTextAlignmentLeft;
            label.textColor = [UIColor lightGrayColor];
            label.backgroundColor = [UIColor blackColor];
            label.numberOfLines = 0;
            label.font = smallFont;
            label.text = rebuiltURL;
            [self addSubview:label];

            yOffset += labelSize.height + kBuffer;
        }


        // My IP Address.
        NSString* ip = [self getIPAddress];
        NSString* ipString = [NSString stringWithFormat:@"Device IP: %@", ip];

        labelSize = [ipString sizeWithFontSafe:largeFont];
        label   = [[UILabel alloc] initWithFrame:CGRectMake( xOffset, yOffset, labelSize.width, labelSize.height ) ];
        label.textAlignment =  NSTextAlignmentLeft;
        label.textColor = [UIColor whiteColor];
        label.backgroundColor = [UIColor blackColor];
        label.numberOfLines = 0;
        label.font = largeFont;
        label.text = ipString;
        [self addSubview:label];

	}
	return self;
}

@end

    
@implementation IOSLaunchDaemonViewController

/**
 * The ViewController was created, so now we need to create our view to be controlled.
 */
- (void) loadView
{
    CGRect Frame = [[UIScreen mainScreen] bounds];
    if (![IOSAppDelegate GetDelegate].bDeviceInPortraitMode)
    {
        Swap<float>(Frame.size.width, Frame.size.height);
    }

	self.view = [[IOSLaunchDaemonView alloc] initWithFrame:Frame];

	// settings copied from InterfaceBuilder
	self.edgesForExtendedLayout = UIRectEdgeNone;

    self.wantsFullScreenLayout = YES;
	self.view.clearsContextBeforeDrawing = NO;
	self.view.multipleTouchEnabled = NO;
}


/**
 * Tell the OS that our view controller can auto-rotate between the two landscape modes
 */
- (BOOL)shouldAutorotateToInterfaceOrientation:(UIInterfaceOrientation)interfaceOrientation
{
	return YES;
}

/**
 * Tell the OS to hide the status bar (iOS 7 method for hiding)
 */
- (BOOL)prefersStatusBarHidden
{
	return YES;
}

@end
