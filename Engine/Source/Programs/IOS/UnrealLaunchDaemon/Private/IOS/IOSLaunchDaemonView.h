// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#import <UIKit/UIKit.h>
#import <OpenGLES/EAGL.h>

@interface IOSLaunchDaemonView : UIView
{
@public
    UIActivityIndicatorView*    pSpinner;
	//NSTextStorage*				pTextStorage;
}
@end


@interface IOSLaunchDaemonViewController : UIViewController
{
}
@end