// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealLaunchDaemonApp.h"

#include "IMessageContext.h"
#include "LaunchDaemonMessageHandler.h"
#include "LaunchDaemonMessages.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"

#import <TargetConditionals.h>


void FLaunchDaemonMessageHandler::Init()
{
	MessageEndpoint = FMessageEndpoint::Builder("FLaunchDaemonMessageHandler")
		.Handling<FIOSLaunchDaemonPing>(this, &FLaunchDaemonMessageHandler::HandlePingMessage)
		.Handling<FIOSLaunchDaemonLaunchApp>(this, &FLaunchDaemonMessageHandler::HandleLaunchRequest);

	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Subscribe<FIOSLaunchDaemonPing>();
	}
}


void FLaunchDaemonMessageHandler::Shutdown()
{
	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint.Reset();
	}
}


void FLaunchDaemonMessageHandler::HandlePingMessage(const FIOSLaunchDaemonPing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (MessageEndpoint.IsValid())
	{
		FMessageAddress MessageSender = Context->GetSender();

		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FIOSLaunchDaemonPong>(FString(FPlatformProperties::PlatformName()) + (TARGET_IPHONE_SIMULATOR ? FString(TEXT("Simulator:")) : FString(TEXT("@"))) + FString(FPlatformProcess::ComputerName())
			, FPlatformProcess::ComputerName()
			, "ULD_Ready"
			, UI_USER_INTERFACE_IDIOM() == UIUserInterfaceIdiomPhone? "Phone" : "Tablet"
			, false
			, false
			, false
			), MessageSender);
	}
}


void FLaunchDaemonMessageHandler::HandleLaunchRequest(const FIOSLaunchDaemonLaunchApp& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FString Error;
	// Leaving in extra variable in case text debugging needs to be injected.
	FString LaunchURL = FString(Message.AppID + TEXT("://") + Message.Parameters);
	Launch(LaunchURL);
}


void FLaunchDaemonMessageHandler::Launch(const FString& LaunchURL)
{
	NSUserDefaults *defaults = [NSUserDefaults standardUserDefaults];	
	[defaults setObject:[NSString stringWithCString:TCHAR_TO_ANSI(*LaunchURL) encoding:NSASCIIStringEncoding] forKey:@"PreviousLaunchURL"];
	[defaults synchronize];
	NSLog(@"Data saved\n");

	FString Error;
	FPlatformProcess::LaunchURL(*LaunchURL, NULL, &Error);

	// Exiting the process prevents a sockets conflict with the game. 
	// Here's the supported sequence of events:
	// 1) Launch ULD manually to kick things off
	// 2) UFE wants to launch the game and sends the LaunchRequest.
	// 3a) If ULD is running, it launches the game and shuts down
	// 3b) If the game is running, it launches ULD with special arguments and shuts down. 
	//     ULD detects that it needs to do an immediate launch, and after a few second delay to let the game shut down,
	//     it relaunches the game.
	// 4) rinse and repeat.
	//
	// It's worth mentioning that exit(0) is not considered an appropriate way for shutting down a consumer app. But in this case, it works fine.
	exit(0);
}
