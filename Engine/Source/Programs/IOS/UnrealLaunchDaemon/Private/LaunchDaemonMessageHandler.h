// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

class FIOSLaunchDaemonLaunchApp;
class FIOSLaunchDaemonPing;
class FMessageEndpoint;
class IMessageContext;


class FLaunchDaemonMessageHandler
{
public:

	void Init();
	void Shutdown();
	void Launch(const FString& LaunchURL);

private:

	void HandlePingMessage(const FIOSLaunchDaemonPing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void HandleLaunchRequest(const FIOSLaunchDaemonLaunchApp& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	FString AppId;
};
