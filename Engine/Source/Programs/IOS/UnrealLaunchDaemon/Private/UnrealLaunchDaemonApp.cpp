// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealLaunchDaemonApp.h"
#include "RequiredProgramMainCPPInclude.h"
#include "LaunchDaemonMessageHandler.h"

IMPLEMENT_APPLICATION(UnrealLaunchDaemon, "UnrealLaunchDaemon");

DEFINE_LOG_CATEGORY(LogLaunchDaemon);

