// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealLaunchDaemon : ModuleRules
{
	public UnrealLaunchDaemon( ReadOnlyTargetRules Target ) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"ApplicationCore",
				"NetworkFile",
				"Projects",
				"StreamingFile",
				"Slate",
				"SlateCore",
				"StandaloneRenderer",
				"LaunchDaemonMessages",
				"Projects",
				"Messaging",
				"UdpMessaging"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Messaging",
			}
		);
	}
}
