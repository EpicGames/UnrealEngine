// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms("IOS")]
public class UnrealLaunchDaemonTarget : TargetRules
{
	public UnrealLaunchDaemonTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		bUsesSlate = false;
		//PlatformType = TargetRules.TargetPlatformType.Mobile;
		//bRequiresUnrealHeaderGeneration = true;
		AdditionalPlugins.Add("UdpMessaging");

		LaunchModuleName = "UnrealLaunchDaemon";

		bBuildDeveloperTools = false;
		bCompileAgainstEngine = false;

		bHasExports = false;
	}
}
