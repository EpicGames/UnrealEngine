/**
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections.Generic;
using System.Text;
using System.Security.Cryptography.X509Certificates;
using System.Security.Cryptography;
using System.IO;
using MachObjectHandling;
using System.Diagnostics;
using System.Xml;
using System.Linq;

namespace iPhonePackager
{
	public class CodeSignatureBuilder
	{
		/// <summary>
		/// The file system used to read/write different parts of the application bundle
		/// </summary>
		public FileOperations.FileSystemAdapter FileSystem;

		/// <summary>
		/// The contents of Info.plist, mutable after PrepareForSigning is called, until PerformSigning is called, where it is flattened back to bytes
		/// </summary>
		public Utilities.PListHelper Info = null;

		/// <summary>
		/// The certificate used for code signing
		/// </summary>
		public X509Certificate2 SigningCert;

		/// <summary>
		/// The mobile provision used to sign the application
		/// </summary>
		protected MobileProvision Provision;

		/// <summary>
		/// Returns null if successful, or an error string if it failed
		/// </summary>
		public static string InstallCertificate(string CertificateFilename, string PrivateKeyFilename)
		{
			try
			{
				// Load the certificate
				string CertificatePassword = "";
				X509Certificate2 Cert = new X509Certificate2(CertificateFilename, CertificatePassword, X509KeyStorageFlags.MachineKeySet | X509KeyStorageFlags.PersistKeySet);
				if (!Cert.HasPrivateKey)
				{
					return "Error: Certificate does not include a private key and cannot be used to code sign";
				}

				// Add the certificate to the store
				X509Store Store = new X509Store();
				Store.Open(OpenFlags.ReadWrite);
				Store.Add(Cert);
				Store.Close();
			}
			catch (Exception ex)
			{
				string ErrorMsg = String.Format("Failed to load or install certificate with error '{0}'", ex.Message);
				Program.Error(ErrorMsg);
				return ErrorMsg;
			}

			return null;
		}

		/// <summary>
		/// Makes sure the required files for code signing exist and can be found
		/// </summary>
		public static bool FindRequiredFiles(out MobileProvision Provision, out X509Certificate2 Cert, out bool bHasOverridesFile, out bool bNameMatch, bool bNameCheck = true)
		{
			Provision = null;
			Cert = null;
			bHasOverridesFile = File.Exists(Config.GetPlistOverrideFilename());
			bNameMatch = false;

			string CFBundleIdentifier = Config.OverrideBundleName;

			if (string.IsNullOrEmpty(CFBundleIdentifier))
			{
				// Load Info.plist, which guides nearly everything else
				string plistFile = Config.EngineBuildDirectory + "/UnrealGame-Info.plist";
				if (!string.IsNullOrEmpty(Config.ProjectFile))
				{
					plistFile = Path.GetDirectoryName(Config.ProjectFile) + "/Intermediate/" + Config.OSString + "/" + Path.GetFileNameWithoutExtension(Config.ProjectFile) + "-Info.plist";

					if (!File.Exists(plistFile))
					{
						plistFile = Config.IntermediateDirectory + "/UnrealGame-Info.plist";
						if (!File.Exists(plistFile))
						{
							plistFile = Config.EngineBuildDirectory + "/UnrealGame-Info.plist";
						}
					}
				}
				Utilities.PListHelper Info = null;
				try
				{
					string RawInfoPList = File.ReadAllText(plistFile, Encoding.UTF8);
					Info = new Utilities.PListHelper(RawInfoPList); ;
				}
				catch (Exception ex)
				{
					Console.WriteLine(ex.Message);
				}

				if (Info == null)
				{
					return false;
				}

				// Get the name of the bundle
				Info.GetString("CFBundleIdentifier", out CFBundleIdentifier);
				if (CFBundleIdentifier == null)
				{
					return false;
				}
				else
				{
					CFBundleIdentifier = CFBundleIdentifier.Replace("${BUNDLE_IDENTIFIER}", Path.GetFileNameWithoutExtension(Config.ProjectFile));
				}
			}

			// Check for a mobile provision
			try
			{
				string MobileProvisionFilename = MobileProvision.FindCompatibleProvision(CFBundleIdentifier, out bNameMatch);
				Provision = MobileProvisionParser.ParseFile(MobileProvisionFilename);
			}
			catch (Exception)
			{
			}

			// if we have a null provision see if we can find a compatible provision without checking for a certificate
			if (Provision == null)
			{
				try
				{
					string MobileProvisionFilename = MobileProvision.FindCompatibleProvision(CFBundleIdentifier, out bNameMatch, false);
					Provision = MobileProvisionParser.ParseFile(MobileProvisionFilename);
				}
				catch (Exception)
				{
				}

				// if we have a null provision see if we can find a valid provision without checking for name match
				if (Provision == null && !bNameCheck)
				{
					try
					{
						string MobileProvisionFilename = MobileProvision.FindCompatibleProvision(CFBundleIdentifier, out bNameMatch, false, false);
						Provision = MobileProvisionParser.ParseFile(MobileProvisionFilename);
					}
					catch (Exception)
					{
					}
				}
			}

			// Check for a suitable signature to match the mobile provision
			if (Provision != null)
			{
				Cert = CodeSignatureBuilder.FindCertificate(Provision);
			}

			return true;
		}

		protected virtual byte[] GetMobileProvision(string CFBundleIdentifier)
		{
			// find the movile provision file in the library
			bool bNameMatch;
			string MobileProvisionFilename = MobileProvision.FindCompatibleProvision(CFBundleIdentifier, out bNameMatch);

			byte[] Result = null;
			try
			{
				Result = File.ReadAllBytes(MobileProvisionFilename);
			}
			catch (System.Exception ex)
			{
				Program.Error("Could not find {0}.mobileprovision in {1} (error: '{2}').  Please follow the setup instructions to get a mobile provision from the Apple Developer site.",
					Program.GameName, Path.GetFullPath(Config.BuildDirectory), ex.Message);
				Program.ReturnCode = (int)ErrorCodes.Error_ProvisionNotFound;
				throw ex;
			}

			return Result;
		}

		public void LoadMobileProvision(string CFBundleIdentifier)
		{
			byte[] MobileProvisionFile = GetMobileProvision(CFBundleIdentifier);

			if (MobileProvisionFile != null)
			{
				Provision = MobileProvisionParser.ParseFile(MobileProvisionFile);

				Program.Log("Using mobile provision '{0}' to code sign", Provision.ProvisionName);

				// Re-embed the mobile provision
				FileSystem.WriteAllBytes("embedded.mobileprovision", MobileProvisionFile);
				Program.Log(" ... Writing updated embedded.mobileprovision");
			}
		}

		static private string CertToolData = "";
		static public void OutputReceivedCertToolProcessCall(Object Sender, DataReceivedEventArgs Line)
		{
			if ((Line != null) && !String.IsNullOrEmpty (Line.Data)) {
				CertToolData += Line.Data + "\n";
			}
		}

		/// <summary>
		/// Finds all valid installed certificates
		/// </summary>
		public static void FindCertificates()
		{
			string[] ValidCertificatePrefixes = {"iPhone Developer", "iPhone Distribution", "Apple Development", "Apple Distribution"};
			
			X509Certificate2Collection FoundCerts = new X509Certificate2Collection();

			if (Environment.OSVersion.Platform == PlatformID.Unix || Environment.OSVersion.Platform == PlatformID.MacOSX)
			{
				foreach(string SearchPrefix in ValidCertificatePrefixes)
				{ 
					// run certtool y to get the currently installed certificates
					CertToolData = "";
					Process CertTool = new Process();
					CertTool.StartInfo.FileName = "/usr/bin/security";
					CertTool.StartInfo.UseShellExecute = false;
					CertTool.StartInfo.Arguments = string.Format("find-certificate -a -c \"{0}\" -p", SearchPrefix);
					CertTool.StartInfo.RedirectStandardOutput = true;
					CertTool.StartInfo.StandardOutputEncoding = Encoding.UTF8;
					CertTool.OutputDataReceived += new DataReceivedEventHandler(OutputReceivedCertToolProcessCall);
					CertTool.Start();
					CertTool.BeginOutputReadLine();
					CertTool.WaitForExit();
					if (CertTool.ExitCode == 0)
					{
						string header = "-----BEGIN CERTIFICATE-----\n";
						string footer = "-----END CERTIFICATE-----";
						int start = CertToolData.IndexOf(header);
						while (start != -1)
						{
							start += header.Length;
							int end = CertToolData.IndexOf(footer, start);
							string base64 = CertToolData.Substring(start, (end - start));
							byte[] certData = Convert.FromBase64String(base64);
							X509Certificate2 cert = new X509Certificate2(certData);
							FoundCerts.Add(cert);
							start = CertToolData.IndexOf(header, start);
						}
					}
				}
			}
			else
			{
				// Open the personal certificate store on this machine
				X509Store Store = new X509Store();
				Store.Open(OpenFlags.ReadOnly);

				foreach (string SearchPrefix in ValidCertificatePrefixes)
				{
					FoundCerts.AddRange(Store.Certificates.Find(X509FindType.FindBySubjectName, SearchPrefix, false));
				}

				Store.Close();
			}

			foreach (X509Certificate2 TestCert in FoundCerts)
			{
				DateTime EffectiveDate = TestCert.NotBefore.ToUniversalTime();
				DateTime ExpirationDate = TestCert.NotAfter.ToUniversalTime();
				DateTime Now = DateTime.UtcNow;

				bool bCertTimeIsValid = (EffectiveDate < Now) && (ExpirationDate > Now);
				Program.LogVerbose("CERTIFICATE-Name:{0},Validity:{1},StartDate:{2},EndDate:{3}", CryptoAdapter.GetFriendlyNameFromCert(TestCert), bCertTimeIsValid ? "VALID" : "EXPIRED", EffectiveDate.ToString("o"), ExpirationDate.ToString("o"));
			}
		}

		/// <summary>
		/// Finds all valid installed provisions
		/// </summary>
		public static void FindProvisions(string CFBundleIdentifier)
		{
			if (!Directory.Exists(Config.ProvisionDirectory))
			{
				Program.Error("Could not find provision directory '{0}'.", Config.ProvisionDirectory);
				Program.ReturnCode = (int)ErrorCodes.Error_ProvisionNotFound;
				return;
			}
			// cache the provision library
			string SelectedProvision = "";
			string SelectedCert = "";
			string SelectedFile = "";
			int FoundName = -1;

			Dictionary<string, MobileProvision> ProvisionLibrary = new Dictionary<string, MobileProvision>();

			foreach (string ProvisionFile in Directory.EnumerateFiles(Config.ProvisionDirectory, "*.mobileprovision"))
			{
				MobileProvision Provision = MobileProvisionParser.ParseFile(ProvisionFile);
				ProvisionLibrary.Add(ProvisionFile, Provision);
			}

			// first sort all profiles so we look at newer ones first.
			IEnumerable<string> ProfileKeys = ProvisionLibrary.Select(KV => KV.Key)
				.OrderByDescending(K => ProvisionLibrary[K].CreationDate)
				.ToArray();

			// note - all of this is a near duplicate of code in MobileProvisionUtilities, which other functions
			// in this class call to do similar work! 
			// @todo - unify all of this.

			foreach (string ProvisionFile in ProfileKeys)
			{
				MobileProvision p = ProvisionLibrary[ProvisionFile];

				DateTime EffectiveDate = p.CreationDate;
				DateTime ExpirationDate = p.ExpirationDate;
				DateTime Now = DateTime.UtcNow;

				bool bCertTimeIsValid = (EffectiveDate < Now) && (ExpirationDate > Now);
				bool bValid = false;
				X509Certificate2 Cert = FindCertificate(p);
				if (Cert != null)
				{
					bValid = (Cert.NotBefore.ToUniversalTime() < Now) && (Cert.NotAfter.ToUniversalTime() > Now);
				}
				bool bPassesNameCheck = p.ApplicationIdentifier.Substring(p.ApplicationIdentifierPrefix.Length+1) == CFBundleIdentifier;
				bool bPassesCompanyCheck = false;
				bool bPassesWildCardCheck = false;
				if (p.ApplicationIdentifier.Contains("*"))
				{
					string CompanyName = p.ApplicationIdentifier.Substring(p.ApplicationIdentifierPrefix.Length + 1);
					if (CompanyName != "*")
					{
						CompanyName = CompanyName.Substring(0, CompanyName.LastIndexOf("."));
						bPassesCompanyCheck = CFBundleIdentifier.StartsWith(CompanyName);
					}
					else
					{
						bPassesWildCardCheck = true;
					}
				}
				bool bIsManaged = false;
				if (p.ProvisionName == "iOS Team Provisioning Profile: " + CFBundleIdentifier)
				{
					bIsManaged = true;
				}
				bool bDistribution = ((p.ProvisionedDeviceIDs.Count == 0) && !p.bDebug);
				string Validity = "VALID";
				if (!bCertTimeIsValid)
				{
					Validity = "EXPIRED";
				}
				else if (!bValid)
				{
					Validity = "NO_CERT";
				}
				else if (!bPassesNameCheck && !bPassesWildCardCheck && !bPassesCompanyCheck)
				{
					Validity = "NO_MATCH";
				}
				if (bIsManaged)
				{
					Validity = "MANAGED";
				}
				if ((string.IsNullOrWhiteSpace(SelectedProvision) || FoundName < 2) && Validity == "VALID" && !bDistribution)
				{
					int Prev = FoundName;
					if (bPassesNameCheck)
					{
						FoundName = 2;
					}
					else if (bPassesCompanyCheck && FoundName < 1)
					{
						FoundName = 1;
					}
					else if (bPassesWildCardCheck && FoundName == -1)
					{
						FoundName = 0;
					}
					if (FoundName != Prev)
					{
						SelectedProvision = p.ProvisionName;
						SelectedFile = Path.GetFileName(ProvisionFile);
						SelectedCert = CryptoAdapter.GetFriendlyNameFromCert(Cert);
					}
				}
				Program.LogVerbose("PROVISION-File:{0},Name:{1},Validity:{2},StartDate:{3},EndDate:{4},Type:{5}", Path.GetFileName(ProvisionFile), p.ProvisionName, Validity, EffectiveDate.ToString(), ExpirationDate.ToString(), bDistribution ? "DISTRIBUTION" : "DEVELOPMENT");
			}

			Program.LogVerbose("MATCHED-Provision:{0},File:{1},Cert:{2}", SelectedProvision, SelectedFile, SelectedCert);
		}

		/// <summary>
		/// Tries to find a matching certificate on this machine from the the serial number of one of the
		/// certificates in the mobile provision (the one in the mobileprovision is missing the public/private key pair)
		/// </summary>
		public static X509Certificate2 FindCertificate(MobileProvision ProvisionToWorkFrom)
		{
			Program.LogVerbose("  Looking for a certificate that matches the application identifier '{0}'", ProvisionToWorkFrom.ApplicationIdentifier);

			X509Certificate2 Result = null;

			if (Environment.OSVersion.Platform == PlatformID.Unix || Environment.OSVersion.Platform == PlatformID.MacOSX)
			{
				// run certtool y to get the currently installed certificates
				CertToolData = "";
				Process CertTool = new Process ();
				CertTool.StartInfo.FileName = "/usr/bin/security";
				CertTool.StartInfo.UseShellExecute = false;
				CertTool.StartInfo.Arguments = "find-identity -p codesigning -v";
				CertTool.StartInfo.RedirectStandardOutput = true;
				CertTool.StartInfo.StandardOutputEncoding = Encoding.UTF8;
				CertTool.OutputDataReceived += new DataReceivedEventHandler (OutputReceivedCertToolProcessCall);
				CertTool.Start ();
				CertTool.BeginOutputReadLine ();
				CertTool.WaitForExit ();

				Program.LogVerbose("  Running {0} {1}\n{2}", CertTool.StartInfo.FileName, CertTool.StartInfo.Arguments, CertToolData);

				if (CertTool.ExitCode == 0)
				{
					Program.LogVerbose("  Provisioning profile contains the following certificate hashes:");
					foreach (X509Certificate2 SourceCert in ProvisionToWorkFrom.DeveloperCertificates)
					{
						X509Certificate2 ValidInTimeCert = null;
						// see if certificate can be found by serial number
						string CertHash = SourceCert.GetCertHashString();

						Program.LogVerbose("    {0}", CertHash);

						if (CertToolData.Contains (CertHash))
						{
							ValidInTimeCert = SourceCert;
						}

						if (ValidInTimeCert != null)
						{
							// Found a cert in the valid time range, quit now!
							Result = ValidInTimeCert;
							Program.LogVerbose("  Matches!");
							break;
						}
					}
				}
			}
			else
			{
				// Open the personal certificate store on this machine
				X509Store Store = new X509Store ();
				Store.Open (OpenFlags.ReadOnly);

				// Try finding a matching certificate from the serial number (the one in the mobileprovision is missing the public/private key pair)
				foreach (X509Certificate2 SourceCert in ProvisionToWorkFrom.DeveloperCertificates)
				{
					X509Certificate2Collection FoundCerts = Store.Certificates.Find (X509FindType.FindBySerialNumber, SourceCert.SerialNumber, false);

					Program.LogVerbose ("  .. Provision entry SN '{0}' matched {1} installed certificate(s)", SourceCert.SerialNumber, FoundCerts.Count);

					X509Certificate2 ValidInTimeCert = null;
					foreach (X509Certificate2 TestCert in FoundCerts)
					{
						DateTime EffectiveDate = TestCert.NotBefore.ToUniversalTime();
						DateTime ExpirationDate = TestCert.NotAfter.ToUniversalTime();
						DateTime Now = DateTime.UtcNow;

						bool bCertTimeIsValid = (EffectiveDate < Now) && (ExpirationDate > Now);

						Program.LogVerbose ("  .. .. Installed certificate '{0}' is {1} (range '{2}' to '{3}')", CryptoAdapter.GetFriendlyNameFromCert(TestCert), bCertTimeIsValid ? "valid (choosing it)" : "EXPIRED", TestCert.GetEffectiveDateString (), TestCert.GetExpirationDateString ());
						if (bCertTimeIsValid)
						{
							ValidInTimeCert = TestCert;
							break;
						}
					}

					if (ValidInTimeCert != null)
					{
						// Found a cert in the valid time range, quit now!
						Result = ValidInTimeCert;
						break;
					}
				}

				Store.Close ();
			}

			if (Result == null)
			{
				Program.LogVerbose("  .. Failed to find a valid certificate that was in date");
			}

			return Result;
		}

		/// <summary>
		/// Installs a list of certificates to the local store
		/// </summary>
		/// <param name="CertFilenames"></param>
		public static void InstallCertificates(List<string> CertFilenames)
		{
			// Open the personal certificate store on this machine
			X509Store Store = new X509Store();
			Store.Open(OpenFlags.ReadWrite);

			// Install the trust chain certs
			string CertificatePassword = "";
			foreach (string AdditionalCertFilename in CertFilenames)
			{
				if (File.Exists(AdditionalCertFilename))
				{
					X509Certificate2 AdditionalCert = new X509Certificate2(AdditionalCertFilename, CertificatePassword, X509KeyStorageFlags.MachineKeySet);
					Store.Add(AdditionalCert);
				}
				else
				{
					string ErrorMessage = String.Format("... Failed to find an additional certificate '{0}' that is required for code signing", AdditionalCertFilename);
					Program.Error(ErrorMessage);
					throw new FileNotFoundException(ErrorMessage);
				}
			}

			// Close (save) the certificate store
			Store.Close();
		}

		/// <summary>
		/// Loads the signing certificate
		/// </summary>
		protected virtual X509Certificate2 LoadSigningCertificate()
		{
			return FindCertificate(Provision);
		}

		// Creates an weight resource entry for the resource rules dictionary
		protected Dictionary<string, object> CreateResource(int Weight)
		{
			Dictionary<string, object> Result = new Dictionary<string, object>();
			Result.Add("weight", (double)Weight);
			return Result;
		}

		// Creates an omitted resource entry for the resource rules dictionary
		protected Dictionary<string, object> CreateOmittedResource(int Weight)
		{
			Dictionary<string, object> Result = new Dictionary<string, object>();
			Result.Add("omit", true);
			Result.Add("weight", (double)Weight);
			return Result;
		}

		// Creates an optional resource entry for the resource rules dictionary
		protected Dictionary<string, object> CreateOptionalResource(int Weight)
		{
			Dictionary<string, object> Result = new Dictionary<string, object>();
			Result.Add("optional", true);
			Result.Add("weight", (double)Weight);
			return Result;
		}

		// Creates a nested resource entry for the resource rules dictionary
		protected Dictionary<string, object> CreateNestedResource(int Weight)
		{
			Dictionary<string, object> Result = new Dictionary<string, object>();
			Result.Add("nested", true);
			Result.Add("weight", (double)Weight);
			return Result;
		}

		protected virtual byte[] CreateCodeResourcesDirectory(string CFBundleExecutable)
		{
			// Create a rules dict that includes (by wildcard) everything but Info.plist and the rules file
			Dictionary<string, object> Rules1 = new Dictionary<string, object>();
			Rules1.Add("^", true);
			Rules1.Add("^.*\\.lproj/", CreateOptionalResource(1000));
			Rules1.Add("^.*\\.lproj/locversion.plist$", CreateOmittedResource(1100));
			Rules1.Add("^Base\\.lproj", CreateResource(1010));
			Rules1.Add("^version.plist$", true);

			Dictionary<string, object> Rules2 = new Dictionary<string, object>();
			Rules2.Add(".*\\.dSYM($|/)", CreateResource(11));
			Rules2.Add("^", CreateResource(20));
			Rules2.Add("^(.*/)?\\.DS_Store$", CreateOmittedResource(2000));
			Rules2.Add("^(Frameworks|SharedFrameworks|PlugIns|Extensions|Plug-ins|XPCServices|Helpers|MacOS|Library/(Automator|Spotlight|LoginItems))/", CreateNestedResource(10));
			Rules2.Add("^.*", true);
			Rules2.Add("^.*\\.lproj/", CreateOptionalResource(1000));
			Rules2.Add("^.*\\.lproj/locversion.plist$", CreateOmittedResource(1100));
			Rules2.Add("^Base\\.lproj", CreateResource(1010));
			Rules2.Add("^Info\\.plist$", CreateOmittedResource(20));
			Rules2.Add("^PkgInfo$", CreateOmittedResource(20));
			Rules2.Add("^[^/]+$", CreateNestedResource(10));
			Rules2.Add("^embedded\\.provisionprofile$", CreateResource(20));
			Rules2.Add("^version\\.plist$", CreateResource(20));

			// Create the full list of files to exclude (some files get excluded by 'magic' even though they aren't listed as special by rules)
			Dictionary<string, object> TrueExclusionList1 = new Dictionary<string, object>();
			TrueExclusionList1.Add(CFBundleExecutable, null);
			TrueExclusionList1.Add("CodeResources", null);
			TrueExclusionList1.Add("_CodeSignature/CodeResources", null);

			Dictionary<string, object> TrueExclusionList2 = new Dictionary<string, object>();
			TrueExclusionList2.Add("Info.plist", null);
			TrueExclusionList2.Add("PkgInfo", null);
			TrueExclusionList2.Add(CFBundleExecutable, null);
			TrueExclusionList2.Add("CodeResources", null);
			TrueExclusionList2.Add("_CodeSignature/CodeResources", null);

			// Hash each file
			IEnumerable<string> FileList = FileSystem.GetAllPayloadFiles();
			SHA1CryptoServiceProvider Hash1Provider = new SHA1CryptoServiceProvider();
			SHA256CryptoServiceProvider Hash2Provider = new SHA256CryptoServiceProvider();

			Utilities.PListHelper HashedFileEntries1 = new Utilities.PListHelper();
			Utilities.PListHelper HashedFileEntries2 = new Utilities.PListHelper();
			foreach (string Filename in FileList)
			{
				if (!TrueExclusionList1.ContainsKey(Filename))
				{
					System.IO.Stream Stream = FileSystem.OpenRead(Filename);
					byte[] HashData = Hash1Provider.ComputeHash(Stream);
					FileSystem.CloseFile(Stream);
					HashedFileEntries1.AddKeyValuePair(Filename, HashData);
				}
				if (!TrueExclusionList2.ContainsKey(Filename))
				{
					System.IO.Stream Stream = FileSystem.OpenRead(Filename);
					byte[] Hash1Data = Hash1Provider.ComputeHash(Stream);
					FileSystem.CloseFile(Stream);
					Stream = FileSystem.OpenRead(Filename);
					byte[] Hash2Data = Hash2Provider.ComputeHash(Stream);
					FileSystem.CloseFile(Stream);
					Utilities.PListHelper HashEntries = new Utilities.PListHelper();
					HashEntries.AddKeyValuePair("hash", Hash1Data);
					HashEntries.AddKeyValuePair("hash2", Hash2Data);
					HashedFileEntries2.AddKeyValuePair(Filename, HashEntries);
				}
			}

			// Create the CodeResources file that will contain the hashes
			Utilities.PListHelper CodeResources = new Utilities.PListHelper();
			CodeResources.AddKeyValuePair("files", HashedFileEntries1);
			CodeResources.AddKeyValuePair("files2", HashedFileEntries2);
			CodeResources.AddKeyValuePair("rules", Rules1);
			CodeResources.AddKeyValuePair("rules2", Rules2);

			// Write the CodeResources file out
			string CodeResourcesAsString = CodeResources.SaveToString();
			byte[] CodeResourcesAsBytes = Encoding.UTF8.GetBytes(CodeResourcesAsString);
			FileSystem.WriteAllBytes("_CodeSignature/CodeResources", CodeResourcesAsBytes);

			return CodeResourcesAsBytes;
		}

		protected virtual Utilities.PListHelper LoadInfoPList()
		{
			byte[] RawInfoPList = FileSystem.ReadAllBytes("Info.plist");
			return new Utilities.PListHelper(Encoding.UTF8.GetString(RawInfoPList));
		}

		/// <summary>
		/// Prepares this signer to sign an application
		///   Modifies the following files:
		///	 embedded.mobileprovision
		/// </summary>
		public void PrepareForSigning()
		{
			// Load Info.plist, which guides nearly everything else
			Info = LoadInfoPList();

			// Get the name of the bundle
			string CFBundleIdentifier;
			if (!Info.GetString("CFBundleIdentifier", out CFBundleIdentifier))
			{
				throw new InvalidDataException("Info.plist must contain the key CFBundleIdentifier");
			}

			// Load the mobile provision, which provides entitlements and a partial cert which can be used to find an installed certificate
			LoadMobileProvision(CFBundleIdentifier);
			if (Provision == null)
			{
				return;
			}

			// Install the Apple trust chain certs (required to do a CMS signature with full chain embedded)
			List<string> TrustChainCertFilenames = new List<string>();

			string CertPath = Path.GetFullPath(Config.EngineBuildDirectory);
			TrustChainCertFilenames.Add(Path.Combine(CertPath, "AppleWorldwideDeveloperRelationsCA.pem"));
			TrustChainCertFilenames.Add(Path.Combine(CertPath, "AppleRootCA.pem"));

			InstallCertificates(TrustChainCertFilenames);

			// Find and load the signing cert
			SigningCert = LoadSigningCertificate();
			if (SigningCert == null)
			{
				// Failed to find a cert already installed or to install, cannot proceed any futher
				Program.Error("... Failed to find a certificate that matches the mobile provision to be used for code signing");
				Program.ReturnCode = (int)ErrorCodes.Error_CertificateNotFound;
				throw new InvalidDataException("Certificate not found!");
			}
			else
			{
				Program.Log("... Found matching certificate '{0}' (valid from {1} to {2})", CryptoAdapter.GetFriendlyNameFromCert(SigningCert), SigningCert.GetEffectiveDateString(), SigningCert.GetExpirationDateString());
			}
		}

		/// <summary>
		/// Creates an entitlements blob string from the entitlements structure in the mobile provision, merging in an on disk file if it is present.
		/// </summary>
		private string BuildEntitlementString(string CFBundleIdentifier, out string TeamIdentifier)
		{
			// Load the base entitlements string from the mobile provision
			string ProvisionEntitlements = Provision.GetEntitlementsString(CFBundleIdentifier, out TeamIdentifier);

			// See if there is an override entitlements file on disk
			string UserOverridesEntitlementsFilename = FileOperations.FindPrefixedFile(Config.BuildDirectory, Program.GameName + ".entitlements");
			if (File.Exists(UserOverridesEntitlementsFilename))
			{
				// Merge in the entitlements from the on disk file as overrides
				Program.Log("Merging override entitlements from {0} into provision specified entitlements", Path.GetFileName(UserOverridesEntitlementsFilename));

				Utilities.PListHelper Merger = new Utilities.PListHelper(ProvisionEntitlements);
				string Overrides = File.ReadAllText(UserOverridesEntitlementsFilename, Encoding.UTF8);
				Merger.MergePlistIn(Overrides);

				return Merger.SaveToString();
			}
			else
			{
				// The ones from the mobile provision need no overrides
				return ProvisionEntitlements;
			}
		}


		/// <summary>
		/// Does the actual work of signing the application
		///   Modifies the following files:
		///	 Info.plist
		///	 [Executable] (file name derived from CFBundleExecutable in the Info.plist, e.g., UnrealGame)
		///	 _CodeSignature/CodeResources
		///	 [ResourceRules] (file name derived from CFBundleResourceSpecification, e.g., CustomResourceRules.plist)
		/// </summary>
		public void PerformSigning()
		{
			DateTime SigningTime = DateTime.Now;

			// Get the name of the executable file
			string CFBundleExecutable;
			if (!Info.GetString("CFBundleExecutable", out CFBundleExecutable))
			{
				throw new InvalidDataException("Info.plist must contain the key CFBundleExecutable");
			}

			// Get the name of the bundle
			string CFBundleIdentifier;
			if (!Info.GetString("CFBundleIdentifier", out CFBundleIdentifier))
			{
				throw new InvalidDataException("Info.plist must contain the key CFBundleIdentifier");
			}

			// Save the Info.plist out
			byte[] RawInfoPList = Encoding.UTF8.GetBytes(Info.SaveToString());
			Info.SetReadOnly(true);
			FileSystem.WriteAllBytes("Info.plist", RawInfoPList);
			Program.Log(" ... Writing updated Info.plist");

			// Create the code resources file and load it
			byte[] ResourceDirBytes = CreateCodeResourcesDirectory(CFBundleExecutable);

			// Open the executable
			Program.Log("Opening source executable...");
			byte[] SourceExeData = FileSystem.ReadAllBytes(CFBundleExecutable);

			FatBinaryFile FatBinary = new FatBinaryFile();
			FatBinary.LoadFromBytes(SourceExeData);

			//@TODO: Verify it's an executable (not an object file, etc...)
			ulong CurrentStreamOffset = 0;
			int OverSize = 1024 * 1024;
			// Allocate a large additional buffer to account for the fact that signature generated by IPP/MacOS 15 or older
			// are significantly larger than the cones generated by MacOS 26's codesign
			byte[] FinalExeData = new byte[SourceExeData.Length + 4 * OverSize];
			int ArchIndex = 0;

			foreach (MachObjectFile Exe in FatBinary.MachObjectFiles)
			{
				Program.Log("... Processing one mach object (binary is {0})", FatBinary.bIsFatBinary ? "fat" : "thin");

				// Pad the memory stream with extra room to handle any possible growth in the code signing data
				int ExeSize = (FatBinary.bIsFatBinary ? (int)FatBinary.Archs[ArchIndex].Size : SourceExeData.Length);
				MemoryStream OutputExeStream = new MemoryStream(ExeSize + OverSize);

				// Copy the data up to the executable into the final stream
				if (FatBinary.bIsFatBinary)
				{
					if (ArchIndex == 0)
					{
						OutputExeStream.Seek(0, SeekOrigin.Begin);
						OutputExeStream.Write(SourceExeData, (int)CurrentStreamOffset, (int)FatBinary.Archs[ArchIndex].Offset - (int)CurrentStreamOffset);
						OutputExeStream.Seek(0, SeekOrigin.Begin);

						byte[] HeaderData = OutputExeStream.ToArray();
						HeaderData.CopyTo(FinalExeData, (long)CurrentStreamOffset);
						CurrentStreamOffset += (ulong)HeaderData.Length;
					}
					else
					{
						byte[] ZeroData = new byte[(int)FatBinary.Archs[ArchIndex].Offset - (int)CurrentStreamOffset];
						ZeroData.CopyTo(FinalExeData, (long)CurrentStreamOffset);
						CurrentStreamOffset += (ulong)ZeroData.Length;
					}
				}

				// Copy the executable into the stream
				int ExeOffset = (FatBinary.bIsFatBinary ? (int)FatBinary.Archs[ArchIndex].Offset : 0);
				OutputExeStream.Seek(0, SeekOrigin.Begin);
				OutputExeStream.Write(SourceExeData, ExeOffset, ExeSize);
				OutputExeStream.Seek(0, SeekOrigin.Begin);
				long Length = OutputExeStream.Length;

				// Find out if there was an existing code sign blob and find the linkedit segment command
				MachLoadCommandCodeSignature CodeSigningBlobLC = null;
				MachLoadCommandSegment LinkEditSegmentLC = null;
				MachLoadCommandSegment TextSegmentLC = null;
				foreach (MachLoadCommand Command in Exe.Commands)
				{
					if (CodeSigningBlobLC == null)
					{
						CodeSigningBlobLC = Command as MachLoadCommandCodeSignature;
					}

					if (LinkEditSegmentLC == null)
					{
						LinkEditSegmentLC = Command as MachLoadCommandSegment;
						if (LinkEditSegmentLC.SegmentName != "__LINKEDIT")
						{
							LinkEditSegmentLC = null;
						}
					}

					if (TextSegmentLC == null)
					{
						TextSegmentLC = Command as MachLoadCommandSegment;
						if (TextSegmentLC.SegmentName != "__TEXT")
						{
							TextSegmentLC = null;
						}
					}
				}

				if (LinkEditSegmentLC == null)
				{
					throw new InvalidDataException("Did not find a Mach segment load command for the __LINKEDIT segment");
				}

				// If the existing code signing blob command is missing, make sure there is enough space to add it
				// Insert the code signing blob if it isn't present
				//@TODO: Insert the code signing blob if it isn't present
				if (CodeSigningBlobLC == null)
				{
					throw new InvalidDataException("Did not find a Code Signing LC.  Injecting one into a fresh executable is not currently supported.");
				}

				// Verify that the code signing blob is at the end of the linkedit segment (and thus can be expanded if needed)
				if ((CodeSigningBlobLC.BlobFileOffset + CodeSigningBlobLC.BlobFileSize) != (LinkEditSegmentLC.FileOffset + LinkEditSegmentLC.FileSize))
				{
					throw new InvalidDataException("Code Signing LC was present but not at the end of the __LINKEDIT segment, unable to replace it");
				}

				int SignedFileLength = (int)CodeSigningBlobLC.BlobFileOffset;

				// Set ExecSegment data
				UInt64 ExecSegBase = 0;
				UInt64 ExecSegLimit = 0;
				if (TextSegmentLC != null)
				{
					ExecSegBase = TextSegmentLC.FileOffset;
					ExecSegLimit = TextSegmentLC.FileSize;
				}

				// Create the entitlements blob
				string TeamIdentifier = Provision.ApplicationIdentifierPrefix;
				string EntitlementsText = BuildEntitlementString(CFBundleIdentifier, out TeamIdentifier);
				EntitlementsBlob FinalEntitlementsBlob = EntitlementsBlob.Create(EntitlementsText);
				EntitlementsDerBlob FinalEntitlementsDerBlob = EntitlementsDerBlob.Create(EntitlementsText);

				// Create the code directory blob
				CodeDirectoryBlob FinalCodeDirectoryBlob = CodeDirectoryBlob.Create(CFBundleIdentifier, TeamIdentifier, SignedFileLength, ExecSegBase, ExecSegLimit);

				// Create or preserve the requirements blob
				RequirementsBlob FinalRequirementsBlob = null;
				if ((CodeSigningBlobLC != null) && Config.bMaintainExistingRequirementsWhenCodeSigning)
				{
					RequirementsBlob OldRequirements = CodeSigningBlobLC.Payload.GetBlobByMagic(AbstractBlob.CSMAGIC_REQUIREMENTS_TABLE) as RequirementsBlob;
					FinalRequirementsBlob = OldRequirements;
				}

				if (FinalRequirementsBlob == null)
				{
					RequirementBlob.ExpressionOp OldExpression = null;
					if (CodeSigningBlobLC != null)
					{
						RequirementsBlob OldRequirements = CodeSigningBlobLC.Payload.GetBlobByMagic(AbstractBlob.CSMAGIC_REQUIREMENTS_TABLE) as RequirementsBlob;
						if (OldRequirements.GetBlobByKey(0x00003) != null)
						{
							RequirementBlob OldRequire = OldRequirements.GetBlobByKey(0x00003) as RequirementBlob;
							OldExpression = OldRequire.Expression;
						}
					}
					FinalRequirementsBlob = RequirementsBlob.CreateEmpty();
					FinalRequirementsBlob.Add(0x00003, RequirementBlob.CreateFromCertificate(SigningCert, CFBundleIdentifier, OldExpression));
				}

				// Create the code signature blob (which actually signs the code directory)
				CodeDirectorySignatureBlob CodeSignatureBlob = CodeDirectorySignatureBlob.Create();

				// Create the code signature superblob (which contains all of the other signature-related blobs)
				CodeSigningTableBlob CodeSignPayload = CodeSigningTableBlob.Create();
				CodeSignPayload.Add(0x00000, FinalCodeDirectoryBlob);
				CodeSignPayload.Add(0x00002, FinalRequirementsBlob);
				CodeSignPayload.Add(0x00005, FinalEntitlementsBlob);
				CodeSignPayload.Add(0x00007, FinalEntitlementsDerBlob);
				CodeSignPayload.Add(0x10000, CodeSignatureBlob);


				// The ordering of the following steps (and doing the signature twice below) must be preserved.
				// The reason is there are some chicken-and-egg issues here:
				//   The code directory stores a hash of the header, but
				//   The header stores the size of the __LINKEDIT section, which is where the signature blobs go, but
				//   The CMS signature blob signs the code directory
				//
				// So, we need to know the size of a signature blob in order to write a header that is itself hashed
				// and signed by the signature blob

				// Do an initial signature just to get the size
				Program.Log("... Initial signature step ({0:0.00} s elapsed so far)", (DateTime.Now - SigningTime).TotalSeconds);
				CodeSignatureBlob.SignCodeDirectory(SigningCert, SigningTime, FinalCodeDirectoryBlob);

				// Compute the size of everything, and push it into the EXE header
				byte[] DummyPayload = CodeSignPayload.GetBlobBytes();

				// Adjust the header and load command to have the correct size for the code sign blob
				WritingContext OutputExeContext = new WritingContext(new BinaryWriter(OutputExeStream));

				long BlobLength = DummyPayload.Length;

				long NonCodeSigSize = (long)LinkEditSegmentLC.FileSize - CodeSigningBlobLC.BlobFileSize;
				long BlobStartPosition = NonCodeSigSize + (long)LinkEditSegmentLC.FileOffset;

				LinkEditSegmentLC.PatchFileLength(OutputExeContext, (uint)(NonCodeSigSize + BlobLength));
				CodeSigningBlobLC.PatchPositionAndSize(OutputExeContext, (uint)BlobStartPosition, (uint)BlobLength);

				// Now that the executable loader command has been inserted and the appropriate section modified, compute all the hashes
				Program.Log("... Computing hashes ({0:0.00} s elapsed so far)", (DateTime.Now - SigningTime).TotalSeconds);
				OutputExeContext.Flush();

				// Fill out the special hashes
				FinalCodeDirectoryBlob.GenerateSpecialSlotHash(CodeDirectoryBlob.cdInfoSlot, RawInfoPList);
				FinalCodeDirectoryBlob.GenerateSpecialSlotHash(CodeDirectoryBlob.cdRequirementsSlot, FinalRequirementsBlob.GetBlobBytes());
				FinalCodeDirectoryBlob.GenerateSpecialSlotHash(CodeDirectoryBlob.cdResourceDirSlot, ResourceDirBytes);
				FinalCodeDirectoryBlob.GenerateSpecialSlotHash(CodeDirectoryBlob.cdApplicationSlot);
				FinalCodeDirectoryBlob.GenerateSpecialSlotHash(CodeDirectoryBlob.cdEntitlementSlot, FinalEntitlementsBlob.GetBlobBytes());
				FinalCodeDirectoryBlob.GenerateSpecialSlotHash(CodeDirectoryBlob.cdUnknown6Slot);
				FinalCodeDirectoryBlob.GenerateSpecialSlotHash(CodeDirectoryBlob.cdDerEntitlementSlot, FinalEntitlementsDerBlob.GetBlobBytes());

				// Fill out the regular hashes
				FinalCodeDirectoryBlob.ComputeImageHashes(OutputExeStream.ToArray());

				// And compute the final signature
				Program.Log("... Final signature step ({0:0.00} s elapsed so far)", (DateTime.Now - SigningTime).TotalSeconds);
				CodeSignatureBlob.SignCodeDirectory(SigningCert, SigningTime, FinalCodeDirectoryBlob);

				// Generate the signing blob and place it in the output (verifying it didn't change in size)
				byte[] FinalPayload = CodeSignPayload.GetBlobBytes();

				if (DummyPayload.Length != FinalPayload.Length)
				{
					throw new InvalidDataException("CMS signature blob changed size between practice run and final run, unable to create useful code signing data");
				}

				if (FinalPayload.Length > CodeSigningBlobLC.BlobFileSize)
				{
					throw new InvalidDataException("CMS signature blob size is too large for the allocated size, unable to create useful code signing data");
				}

				byte[] ZeroFill = new byte[CodeSigningBlobLC.BlobFileSize - FinalPayload.Length];

				OutputExeContext.PushPositionAndJump(BlobStartPosition);
				OutputExeContext.Write(FinalPayload);
				OutputExeContext.Write(ZeroFill);
				OutputExeContext.PopPosition();

				// Truncate the data so the __LINKEDIT section extends right to the end
				Program.Log("... Committing all edits ({0:0.00} s elapsed so far)", (DateTime.Now - SigningTime).TotalSeconds);
				OutputExeContext.CompleteWritingAndClose();

				Program.Log("... Truncating/copying final binary", DateTime.Now - SigningTime);
				ulong DesiredExecutableLength = LinkEditSegmentLC.FileSize + LinkEditSegmentLC.FileOffset;

				byte[] Data = OutputExeStream.ToArray();
				Data.CopyTo(FinalExeData, (long)CurrentStreamOffset);
				CurrentStreamOffset += DesiredExecutableLength;

				// update the header if it is a fat binary
				if (FatBinary.bIsFatBinary)
				{
					FatBinary.Archs[ArchIndex].Size = (uint)DesiredExecutableLength;
				}

				// increment the architecture index
				ArchIndex++;
			}

			// re-write the header
			FatBinary.WriteHeader(ref FinalExeData, 0);

			// resize to the finale size
			Array.Resize(ref FinalExeData, (int)CurrentStreamOffset); //@todo: Extend the file system interface so we don't have to copy 20 MB just to truncate a few hundred bytes

			// Save the patched and signed executable
			Program.Log("Saving signed executable... ({0:0.00} s elapsed so far)", (DateTime.Now - SigningTime).TotalSeconds);
			FileSystem.WriteAllBytes(CFBundleExecutable, FinalExeData);

			Program.Log("Finished code signing, which took {0:0.00} s", (DateTime.Now - SigningTime).TotalSeconds);
		}
	}
}
