/*
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Net;
using System.Runtime.InteropServices;
using System.Security.Cryptography.X509Certificates;
using System.Text;
using System.Threading;
using Ionic.Zlib;

namespace iPhonePackager
{
	public class Settings
	{
	}

	/**
	 * Operations done at compile time - these may involve the Mac
	 */
	public class CompileTime
	{
		/**
		 * Location of the Xcode installation on the Mac.  For example: 
		 * "/Applications/Xcode.app/Contents/Developer"
		 */
		private static string XcodeDeveloperDir = "";

		private static string iPhone_SigningDevRootMac = "";

		/// <summary>
		/// The file name (no path) of the temporary mobile provision that will be placed on the remote mac for use in makeapp
		/// </summary>
		private static string MacMobileProvisionFilename;
		private static string MacSigningIdentityFilename;

		private static string MacName = "";
		private static string MacStagingRootDir = "";
		private static string MacBinariesDir = "";
		private static string MacXcodeStagingDir = "";

		/** /MacStagingRootDir/Payload */
		public static string RemoteAppPayloadDirectory
		{
			get { return MacStagingRootDir + "/Payload"; }
		}
		
		/** /MacStagingRootDir/Payload/GameName.app */
		protected static string RemoteAppDirectory
		{
			get { return RemoteAppPayloadDirectory + "/" + Program.GameName + (Program.IsClient ? "Client" : "") + Program.Architecture + ".app"; }
		}


		/** /MacStagingRootDir/Payload/GameName.app/GameName */
		protected static string RemoteExecutablePath
		{
			get { return RemoteAppDirectory + "/" + Program.GameName + (Program.IsClient ? "Client" : "") + Program.Architecture; }
		}

		private static string CurrentBaseXCodeCommandLine;

		/**
		 * @return the commandline used to run Xcode (can add commands like "clean" as needed to the result)
		 */
		static private string GetBaseXcodeCommandline()
		{
			string CmdLine = XcodeDeveloperDir + "usr/bin/xcodebuild" +
					" -project UE4_FromPC.xcodeproj" +
					" -configuration \"" + Program.SchemeConfiguration + "\"" +
					" -target '" + Program.SchemeName + "'";
			
			if (Config.OSString == "IOS")
			{
				CmdLine += " -destination generic/platform=iOS" +" -sdk " + ((Program.Architecture == "-simulator") ? "iphonesimulator" : "iphoneos");
			}
			else
			{
				CmdLine += " -destination generic/platform=tvOS" + " -sdk " + ((Program.Architecture == "-simulator") ? "appletvsimulator" : "appletvos");
			}

			// sign with the Distribution identity when packaging for distribution
			if (Config.bUseRPCUtil)
			{
				CmdLine += String.Format(" CODE_SIGN_IDENTITY=\\\"{0}\\\"", Config.CodeSigningIdentity);

				CmdLine += String.Format(" IPHONEOS_DEPLOYMENT_TARGET=\\\"{0}\\\"", Config.MinOSVersion);
			}
			else
			{
				CmdLine += String.Format(" CODE_SIGN_IDENTITY=\"{0}\"", Config.CodeSigningIdentity);

				CmdLine += String.Format(" IPHONEOS_DEPLOYMENT_TARGET=\"{0}\"", Config.MinOSVersion);
			}

			return CmdLine;
		}

		/** 
		 * Handle the plethora of environment variables required to remote to the Mac
		 */
		static public void ConfigurePaths()
		{
			string MachineName = System.Net.Dns.GetHostName();

			XcodeDeveloperDir = Utilities.GetEnvironmentVariable("ue.XcodeDeveloperDir", "/Applications/Xcode.app/Contents/Developer/");

			// MacName=%ue4.iPhone_SigningServerName%
			MacName = Config.OverrideMacName != null ? Config.OverrideMacName : Utilities.GetEnvironmentVariable("ue.IOSSigningServer", "a1487");
			iPhone_SigningDevRootMac = Config.OverrideDevRoot != null ? Config.OverrideDevRoot : "/UE4/Builds";

			if (!Config.bUseRPCUtil)
			{
				bool Results = SSHCommandHelper.Command(MacName, "xcode-select --print-path", "/usr/bin");
				if (Results)
				{
					XcodeDeveloperDir = (string)SSHCommandHelper.SSHReturn["CommandOutput"] + "/";
					XcodeDeveloperDir = XcodeDeveloperDir.TrimEnd();
				}
			}

			// get the path to mirror into on the Mac
			string BinariesDir = Path.GetFullPath(Path.Combine(Environment.CurrentDirectory, @"..\.."));
			string Root = Path.GetPathRoot(BinariesDir);
			string BranchPath = MachineName + "/" + Root[0].ToString() + "/" + BinariesDir.Substring(Root.Length);
			BranchPath = BranchPath.Replace('\\', '/');

			// similar for the game path (strip off the D:\ tpe root)
			BinariesDir = Path.GetFullPath(Path.Combine(Config.BinariesDirectory, ".."));
			Root = Path.GetPathRoot(BinariesDir);

			string GameBranchPath;
			if (Program.GameName == "UnrealGame")
			{
				GameBranchPath = BranchPath;
			}
			else
			{
				GameBranchPath = MachineName + "/" + Root[0].ToString() + "/" + BinariesDir.Substring(Root.Length);
				GameBranchPath = GameBranchPath.Replace('\\', '/');
			}

			Console.WriteLine("BranchPath = {0} --- GameBranchPath = {1}", BranchPath, GameBranchPath);

			// generate the directories to recursively copy into later on
			MacStagingRootDir = string.Format("{0}/{1}/" + Config.OSString, iPhone_SigningDevRootMac, GameBranchPath);
			MacStagingRootDir = MacStagingRootDir.Replace("//", "/");
			MacBinariesDir = string.Format("{0}/{1}/" + Config.OSString, iPhone_SigningDevRootMac, GameBranchPath);
			MacBinariesDir = MacBinariesDir.Replace("//", "/");
			MacXcodeStagingDir = string.Format("{0}/{1}/" + Config.OSString + "/XcodeSupportFiles", iPhone_SigningDevRootMac, GameBranchPath);
			MacXcodeStagingDir = MacXcodeStagingDir.Replace("//", "/");

			MacMobileProvisionFilename = MachineName + "_UE4Temp.mobileprovision";
			MacSigningIdentityFilename = MachineName + "_UE4Temp.p12";
		}

		/// <summary>
		/// Logs out a line of stdout or stderr from RPCUtility.exe to the program log
		/// </summary>
		/// <param name="Sender"></param>
		/// <param name="Line"></param>
		static public void OutputReceivedRemoteProcessCall(Object Sender, DataReceivedEventArgs Line)
		{
			if ((Line != null) && (Line.Data != null) && (Line.Data != ""))
			{
				Program.Log("[RPC] " + Line.Data);

				if (Line.Data.Contains("** BUILD FAILED **"))
				{
					Program.Error("Xcode build failed!");
				}
			}
		}

		static bool FindMobileProvision(string BundleIdentifier, out string OutFileName, bool bCheckCert = true)
		{
			bool bNameMatch;
			string ProvisionWithPrefix = MobileProvision.FindCompatibleProvision(BundleIdentifier, out bNameMatch, bCheckCert, true, false);
			if (!File.Exists(ProvisionWithPrefix))
			{
				ProvisionWithPrefix = FileOperations.FindPrefixedFile(Config.BuildDirectory, Program.GameName + ".mobileprovision");
				if (!File.Exists(ProvisionWithPrefix))
				{
					ProvisionWithPrefix = FileOperations.FindPrefixedFile(Config.BuildDirectory + "/NotForLicensees/", Program.GameName + ".mobileprovision");
					if (!File.Exists(ProvisionWithPrefix))
					{
						ProvisionWithPrefix = FileOperations.FindPrefixedFile(Config.EngineBuildDirectory, "UnrealGame.mobileprovision");
						if (!File.Exists(ProvisionWithPrefix))
						{
							ProvisionWithPrefix = FileOperations.FindPrefixedFile(Config.EngineBuildDirectory + "/NotForLicensees/", "UnrealGame.mobileprovision");
							if(!File.Exists(ProvisionWithPrefix))
							{
								OutFileName = null;
								return false;
							}
						}
					}
				}
			}

			OutFileName = ProvisionWithPrefix;
			return true;
		}

		/// <summary>
		/// Export the certificate to a file
		/// </summary>
		static public void ExportCertificate()
		{
			if(Config.ProvisionFile == null)
			{
				Program.Error("Missing -ProvisionFile=... argument");
				return;
			}

			if(Config.OutputCertificate == null)
			{
				Program.Error("Missing -OutputCertificate=... argument");
				return;
			}

			// export the signing certificate to a file
			MobileProvision Provision = MobileProvisionParser.ParseFile(Config.ProvisionFile);
			X509Certificate2 Certificate = CodeSignatureBuilder.FindCertificate(Provision);
            if (Certificate == null)
            {
                Program.Error("Failed to find a valid certificate");
                return;
            }

			byte[] Data = Certificate.Export(System.Security.Cryptography.X509Certificates.X509ContentType.Pkcs12, "A");
			File.WriteAllBytes(Config.OutputCertificate, Data);
		}

		/// <summary>
		/// Copy the files always needed (even in a stub IPA)
		/// </summary>
		static public void CopyFilesNeededForMakeApp()
		{
			// Copy Info.plist over (modifiying it as needed)
			string SourcePListFilename = Utilities.GetPrecompileSourcePListFilename();
			Utilities.PListHelper Info = Utilities.PListHelper.CreateFromFile(SourcePListFilename);
			
			// Edit the plist
			CookTime.UpdateVersion(Info);

			// Write out the <GameName>-Info.plist file to the xcode staging directory
			string TargetPListFilename = Path.Combine(Config.PCXcodeStagingDir, Program.GameName + "-Info.plist");
			Directory.CreateDirectory(Path.GetDirectoryName(TargetPListFilename));
			string OutString = Info.SaveToString();
			OutString = OutString.Replace("${EXECUTABLE_NAME}", Program.GameName);
			OutString = OutString.Replace("${BUNDLE_IDENTIFIER}", Program.GameName.Replace("_", ""));

			byte[] RawInfoPList = Encoding.UTF8.GetBytes(OutString);
			File.WriteAllBytes(TargetPListFilename, RawInfoPList);

			Program.Log("Updating .plist: {0} --> {1}", SourcePListFilename, TargetPListFilename);

			string FinalMobileProvisionFilename = null;
			CurrentBaseXCodeCommandLine = GetBaseXcodeCommandline();
			if (!Config.bAutomaticSigning)
			{
				// Copy the mobile provision file over
				string CFBundleIdentifier = null;
				Info.GetString("CFBundleIdentifier", out CFBundleIdentifier);

				string ProvisionWithPrefix;
				if(!FindMobileProvision(CFBundleIdentifier, out ProvisionWithPrefix))
				{
					Program.Error("Unable to find mobileprovision");
					return;
				}
				FinalMobileProvisionFilename = Path.Combine(Config.PCXcodeStagingDir, MacMobileProvisionFilename);
				FileOperations.CopyRequiredFile(ProvisionWithPrefix, FinalMobileProvisionFilename);

				// make sure this .mobileprovision file is newer than any other .mobileprovision file on the Mac (this file gets multiple games named the same file, 
				// so the time stamp checking can fail when moving between games, a la the buildmachines!)
				File.SetLastWriteTime(FinalMobileProvisionFilename, DateTime.UtcNow);

				// copy the signing certificate over
				// export the signing certificate to a file
				MobileProvision Provision = MobileProvisionParser.ParseFile(ProvisionWithPrefix);
				var Certificate = CodeSignatureBuilder.FindCertificate(Provision);
				byte[] Data = Certificate.Export(System.Security.Cryptography.X509Certificates.X509ContentType.Pkcs12, "A");
				File.WriteAllBytes(Path.Combine(Config.PCXcodeStagingDir, MacSigningIdentityFilename), Data);
				Config.CodeSigningIdentity = Certificate.FriendlyName; // since the pipeline will use a temporary keychain that will contain only this certificate, this should be the only identity that will work

				if (Provision != null)
				{
					Config.bForDistribution = !Provision.bDebug;
				}
				// regenerate command with new found identity
				CurrentBaseXCodeCommandLine = GetBaseXcodeCommandline();

				// get the UUID
				string AllText = File.ReadAllText(FinalMobileProvisionFilename);
				string UUID = "";
				int idx = AllText.IndexOf("<key>UUID</key>");
				if (idx > 0)
				{
					idx = AllText.IndexOf("<string>", idx);
					if (idx > 0)
					{
						idx += "<string>".Length;
						UUID = AllText.Substring(idx, AllText.IndexOf("</string>", idx) - idx);
					}
				}

				CurrentBaseXCodeCommandLine += String.Format(" PROVISIONING_PROFILE=" + UUID);
			}
			else
			{
				CurrentBaseXCodeCommandLine += " DEVELOPMENT_TEAM=" + Config.TeamID + " CODE_SIGN_STYLE=\"Automatic\" -allowProvisioningUpdates";
			}
			
			// look for an entitlements file (optional)
			string SourceEntitlements = FileOperations.FindPrefixedFile(Config.BuildDirectory, Program.GameName + ".entitlements");
			
			// set where to make the entitlements file (
			string TargetEntitlements = Path.Combine(Config.PCXcodeStagingDir, Program.GameName + ".entitlements");
			if (File.Exists(SourceEntitlements))
			{
				FileOperations.CopyRequiredFile(SourceEntitlements, TargetEntitlements);
			}
			else
			{
				// we need to have something so Xcode will compile, so we just set the get-task-allow, since we know the value, 
				// which is based on distribution or not (true means debuggable)
				File.WriteAllText(TargetEntitlements, string.Format("<plist><dict><key>get-task-allow</key><{0}/></dict></plist>",
					Config.bForDistribution ? "false" : "true"));
			}

			string ProjectFile = Config.RootRelativePath + @"Engine\Intermediate\ProjectFiles\UE4.xcodeproj\project.pbxproj";
			if (Program.GameName != "UnrealGame")
			{
				ProjectFile = Path.GetDirectoryName(Config.IntermediateDirectory) + @"\ProjectFiles\" + Program.GameName + @".xcodeproj\project.pbxproj";
			}
			FileOperations.CopyRequiredFile(ProjectFile, Path.Combine(Config.PCXcodeStagingDir, @"project.pbxproj.datecheck"));

			// needs Mac line endings so it can be executed
			string SrcPath = @"..\..\..\Build\" + Config.OSString + @"\XcodeSupportFiles\prepackage.sh";
			string DestPath = Path.Combine(Config.PCXcodeStagingDir, @"prepackage.sh");
			Program.Log(" ... '" + SrcPath + "' -> '" + DestPath + "'");
			string SHContents = File.ReadAllText(SrcPath);
			SHContents = SHContents.Replace("\r\n", "\n");
			File.WriteAllText(DestPath, SHContents);

			CookTime.CopySignedFiles();
		}
		
		/**
		 * Handle spawning of the RPCUtility with parameters
		 */
		public static bool RunRPCUtilty( string RPCCommand, bool bIsSilent = false )
		{
			string CommandLine = "";
			string WorkingFolder = "";
			string DisplayCommandLine = "";
            string TempKeychain = "$HOME/Library/Keychains/UE4TempKeychain.keychain";
            string Certificate = "XcodeSupportFiles/" + MacSigningIdentityFilename;
			string LoginKeychain = "$HOME/Library/Keychains/login.keychain";
			ErrorCodes Error = ErrorCodes.Error_Unknown;

			switch (RPCCommand.ToLowerInvariant())
			{
			case "deletemacstagingfiles":
				Program.Log( " ... deleting staging files on the Mac" );
				DisplayCommandLine = "rm -rf Payload";
				CommandLine = "\"" + MacStagingRootDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacStagingRootDir + "\"";
				break;

			case "ensureprovisiondirexists":
				Program.Log(" ... creating provisioning profiles directory");

				DisplayCommandLine = String.Format("{0} mkdir -p ~/Library/{{PROVISIONPARENTDIR}}/Provisioning\\ Profiles", Config.XcodeGetProvisionParentDirectory);

				CommandLine = "\"" + MacXcodeStagingDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacXcodeStagingDir + "\"";
				break;

			case "installprovision":
				// Note: The provision must have already been copied over to the Mac
				Program.Log(" ... installing .mobileprovision");

				DisplayCommandLine = String.Format("{0} cp -f {1} ~/Library/{{PROVISIONPARENTDIR}}/Provisioning\\ Profiles", Config.XcodeGetProvisionParentDirectory, MacMobileProvisionFilename);

				CommandLine = "\"" + MacXcodeStagingDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacXcodeStagingDir + "\"";
				break;

			case "removeprovision":
				Program.Log(" ... removing .mobileprovision");
				DisplayCommandLine = String.Format("{0} rm -f ~/Library/{{PROVISIONPARENTDIR}}/Provisioning\\ Profiles/{1}", Config.XcodeGetProvisionParentDirectory, MacMobileProvisionFilename);
				CommandLine = "\"" + MacXcodeStagingDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacXcodeStagingDir + "\"";
				break;

			case "setexec": 
				// Note: The executable must have already been copied over
				Program.Log(" ... setting executable bit");
				DisplayCommandLine = "chmod a+x \'" + RemoteExecutablePath + "\'";
				CommandLine = "\"" + MacStagingRootDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacStagingRootDir + "\"";
				break;

			case "prepackage":
				Program.Log(" ... running prepackage script remotely ");
				DisplayCommandLine = String.Format("sh prepackage.sh {0} " + Config.OSString + " {1} {2}", Program.GameName, Program.GameConfiguration, Program.Architecture);
				CommandLine = "\"" + MacXcodeStagingDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacXcodeStagingDir + "\"";
				break;

			case "makeapp":
				Program.Log(" ... making application (codesign, etc...)");
				Program.Log("  Using signing identity '{0}'", Config.CodeSigningIdentity);
                DisplayCommandLine = "security -v unlock-keychain -p \"A\" \"" + TempKeychain + "\" && " + CurrentBaseXCodeCommandLine;
				CommandLine = "\"" + MacXcodeStagingDir + "/..\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacXcodeStagingDir + "/..\"";
				Error = ErrorCodes.Error_RemoteCertificatesNotFound;
				break;

			case "createkeychain":
				Program.Log(" ... creating temporary key chain with signing certificate");
				Program.Log("  Using signing identity '{0}'", Config.CodeSigningIdentity);
				if (Config.bAutomaticSigning)
				{                                                                                                                                                                                                                                                                                                           
						DisplayCommandLine = "security dump-keychain -i login.keychain && security create-keychain -p \"A\" \"" + TempKeychain + "\" && security list-keychains -s \"" + TempKeychain + "\" && security list-keychains && security set-keychain-settings -t 3600 -l  \"" + TempKeychain + "\" && security -v unlock-keychain -p \"A\" \"" + TempKeychain + "\" && security default-keychain -s \"" + TempKeychain + "\" && security import login.keychain -P \"A\" -T /usr/bin/codesign";//&& security set-key-partition-list -S apple-tool:,apple:,codesign: -s -k \"A\" -t private " + TempKeychain;
				}
				else
				{
					DisplayCommandLine = "security create-keychain -p \"A\" \"" + TempKeychain + "\" && security list-keychains -s \"" + TempKeychain + "\" && security list-keychains && security set-keychain-settings -t 3600 -l  \"" + TempKeychain + "\" && security -v unlock-keychain -p \"A\" \"" + TempKeychain + "\" && security import " + Certificate + " -k \"" + TempKeychain + "\" -P \"A\" -T /usr/bin/codesign -T /usr/bin/security -t agg && CERT_IDENTITY=$(security find-identity -v -p codesigning \"" + TempKeychain + "\" | head -1 | grep '\"' | sed -e 's/[^\"]*\"//' -e 's/\".*//') && security default-keychain -s \"" + TempKeychain + "\" && security set-key-partition-list -S apple-tool:,apple:,codesign: -s -k \"A\" -D \"$CERT_IDENTITY\" -t private " + TempKeychain;
				}
                CommandLine = "\"" + MacXcodeStagingDir + "/..\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacXcodeStagingDir + "/..\"";
				break;

            case "deletekeychain":
				Program.Log(" ... remove temporary key chain");
				Program.Log("  Using signing identity '{0}'", Config.CodeSigningIdentity);
				DisplayCommandLine = "security list-keychains -s \"" + LoginKeychain + "\" && security delete-keychain \"" + TempKeychain + "\"";
				CommandLine = "\"" + MacXcodeStagingDir + "/..\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacXcodeStagingDir + "/..\"";
				break;

			case "validation":
				Program.Log( " ... validating distribution package" );
				DisplayCommandLine = XcodeDeveloperDir + "Platforms/iPhoneOS.platform/Developer/usr/bin/Validation " + RemoteAppDirectory;
				CommandLine = "\"" + MacStagingRootDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacStagingRootDir + "\"";
				break;

			case "deleteipa":
				Program.Log(" ... deleting IPA on Mac");
				DisplayCommandLine = "rm -f " + Config.IPAFilenameOnMac;
				CommandLine = "\"" + MacStagingRootDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacStagingRootDir + "\"";
				break;

			case "kill":
				Program.Log( " ... killing" );
				DisplayCommandLine = "killall " + Program.GameName;
				CommandLine = ". " + DisplayCommandLine;
				WorkingFolder = ".";
				break;

			case "strip":
				Program.Log( " ... stripping" );
				DisplayCommandLine = "/usr/bin/xcrun strip '" + RemoteExecutablePath + "'";
				CommandLine = "\"" + MacStagingRootDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacStagingRootDir + "\"";
				break;

			case "resign":
				Program.Log("... resigning");
				DisplayCommandLine = "bash -c '" + "chmod a+x ResignScript" + ";" + "./ResignScript" + "'";
				CommandLine = "\"" + MacStagingRootDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacStagingRootDir + "\"";
				break;

            case "zip":
				Program.Log( " ... zipping" );

				// NOTE: -y preserves symbolic links which is needed for iOS distro builds
				// -x excludes a file (excluding the dSYM keeps sizes smaller, and it shouldn't be in the IPA anyways)
				string dSYMName = "Payload/" + Program.GameName + (Program.IsClient ? "Client" : "") + Program.Architecture + ".app.dSYM";
				DisplayCommandLine = String.Format("zip -q -r -y -{0} -T {1} Payload iTunesArtwork -x {2}/ -x {2}/* " +
					"-x {2}/Contents/ -x {2}/Contents/* -x {2}/Contents/Resources/ -x {2}/Contents/Resources/* " +
					" -x {2}/Contents/Resources/DWARF/ -x {2}/Contents/Resources/DWARF/*",
					(int)Config.RecompressionSetting,
					Config.IPAFilenameOnMac,
					dSYMName);

				CommandLine = "\"" + MacStagingRootDir + "\" " + DisplayCommandLine;
				WorkingFolder = "\"" + MacStagingRootDir + "\"";
				break;

			case "gendsym":
				Program.Log( " ... generating DSYM" );

				string ExePath  = "Payload/" + Program.GameName + (Program.IsClient ? "Client" : "") + ".app/" + Program.GameName + (Program.IsClient ? "Client" : "");
				string dSYMPath = Program.GameName + (Program.IsClient ? "Client" : "") + ".app.dSYM";
				DisplayCommandLine = String.Format("dsymutil -o {0} {1}", dSYMPath, ExePath);

				CommandLine = "\"" + MacStagingRootDir + "\"" + DisplayCommandLine;
				WorkingFolder = "\"" + MacStagingRootDir + "\"";
				break;

			default:
				Program.Error( "Unrecognized RPC command" );
				return ( false );
			}

			Program.Log( " ... working folder: " + WorkingFolder );
			Program.Log( " ... " + DisplayCommandLine );
			Program.Log(" ... full command: " +  MacName + " " + CommandLine);

			bool bSuccess = false;
			if( Config.bUseRPCUtil )
			{
				Program.Log( "Running RPC on " + MacName + " ... " );

				Process RPCUtil = new Process();
				RPCUtil.StartInfo.FileName = @"..\RPCUtility.exe";
				RPCUtil.StartInfo.UseShellExecute = false;
				RPCUtil.StartInfo.Arguments = MacName + " " + CommandLine;
				RPCUtil.StartInfo.RedirectStandardOutput = true;
				RPCUtil.StartInfo.RedirectStandardError = true;
				RPCUtil.OutputDataReceived += new DataReceivedEventHandler(OutputReceivedRemoteProcessCall);
				RPCUtil.ErrorDataReceived += new DataReceivedEventHandler(OutputReceivedRemoteProcessCall);

				RPCUtil.Start();

				RPCUtil.BeginOutputReadLine();
				RPCUtil.BeginErrorReadLine();

				RPCUtil.WaitForExit();

				bSuccess = (RPCUtil.ExitCode == 0);
				if (bSuccess == false && !bIsSilent)
				{
					Program.Error("RPCCommand {0} failed with return code {1}", RPCCommand, RPCUtil.ExitCode);
					switch (RPCCommand.ToLowerInvariant())
					{
						case "installprovision":
							Program.Error("Ensure your access permissions for '~/Library/$PROVISIONPARENTDIR/Provisioning Profiles' are set correctly.");
							break;
						default:
							break;
					}
				}
			}
			else
			{
				Program.Log("Running SSH on " + MacName + " ... ");
				bSuccess = SSHCommandHelper.Command(MacName, DisplayCommandLine, WorkingFolder);
				if (bSuccess == false && !bIsSilent)
				{
					Program.Error("RPCCommand {0} failed with return code {1}", RPCCommand, Error);
					Program.ReturnCode = (int)Error;
				}
			}


			return bSuccess;
		}


		/** 
		 * Creates the application directory on the Mac
		 */
		static public void CreateApplicationDirOnMac()
		{
			DateTime StartTime = DateTime.Now;

			// Cleans out the intermediate folders on both ends
			CompileTime.ExecuteRemoteCommand("DeleteIPA");
			CompileTime.ExecuteRemoteCommand("DeleteMacStagingFiles");
			Program.ExecuteCommand("Clean", null);
			//@TODO: mnoland 10/5/2010
			// Need to come up with a way to prevent this from causing an error on the remote machine
			// CompileTime.ExecuteRemoteCommand("Clean");

			// Stage files
			Program.Log("Staging files before copying to Mac ...");
			CopyFilesNeededForMakeApp();

			// Copy staged files from PC to Mac
			Program.ExecuteCommand("StageMacFiles", null);

            // Set the executable bit on the EXE
            CompileTime.ExecuteRemoteCommand("SetExec");

			// Install the provision (necessary for MakeApp to succeed)
			CompileTime.ExecuteRemoteCommand("EnsureProvisionDirExists");
			CompileTime.ExecuteRemoteCommand("InstallProvision");

			// strip the symbols if desired or required
			if (Config.bForceStripSymbols || Config.bForDistribution)
			{
				CompileTime.ExecuteRemoteCommand("Strip");
			}

			// sign the exe, etc...
			CompileTime.ExecuteRemoteCommand("PrePackage");
			if (!Config.bUseRPCUtil)
			{
                CompileTime.ExecuteRemoteCommand("DeleteKeyChain", true);
                CompileTime.ExecuteRemoteCommand("CreateKeyChain");
			}
			CompileTime.ExecuteRemoteCommand("MakeApp");
            if (!Config.bUseRPCUtil)
            {
                CompileTime.ExecuteRemoteCommand("DeleteKeyChain");
            }

            Program.Log(String.Format("Finished creating .app directory on Mac (took {0:0.00} s)",
				(DateTime.Now - StartTime).TotalSeconds));
		}
		
		/** 
		 * Packages an IPA on the Mac
		 */
		static public void PackageIPAOnMac()
		{
			// Create the .app structure on the Mac (and codesign, etc...)
			CreateApplicationDirOnMac();

			DateTime StartTime = DateTime.Now;

			// zip up
			CompileTime.ExecuteRemoteCommand("Zip");

			// fetch the IPA
			if (Config.bCreateStubSet)
			{
				Program.ExecuteCommand("GetStubIPA", null);
			}
			else
			{
				Program.ExecuteCommand("GetIPA", null);
			}

			Program.Log(String.Format("Finished packaging into IPA (took {0:0.00} s)",
				(DateTime.Now - StartTime).TotalSeconds));
		}

		static public void DangerouslyFastMode()
		{
			CompileTime.ExecuteRemoteCommand("MakeApp");
		}

		static public void ExecuteRemoteCommand(string RemoteCommand, bool bIsSilent = false)
		{
            RunRPCUtilty(RemoteCommand, bIsSilent);
		}

		static public bool ExecuteCompileCommand(string Command, string RPCCommand)
		{
			switch (Command.ToLowerInvariant())
			{
				case "clean":
					Program.Log("Cleaning temporary files from PC ... ");
					Program.Log(" ... cleaning: " + Config.PCStagingRootDir);
					FileOperations.DeleteDirectory(new DirectoryInfo(Config.PCStagingRootDir));
					break;
 
				case "rpc":
					ExecuteRemoteCommand(RPCCommand);
					break;

				case "getipa":
					{
						Program.Log("Fetching IPA from Mac...");

						string IpaDestFilename = Config.GetIPAPath(".ipa");
						FileOperations.DownloadFile(MacName, MacStagingRootDir + "/" + Config.IPAFilenameOnMac, IpaDestFilename);

						Program.Log("... Saved IPA to '{0}'", Path.GetFullPath(IpaDestFilename));
					}
					break;

				case "getstubipa":
					{
						Program.Log("Fetching stub IPA from Mac...");

						string IpaDestFilename = Config.GetIPAPath(".stub");
						FileOperations.DownloadFile(MacName, MacStagingRootDir + "/" + Config.IPAFilenameOnMac, IpaDestFilename);

						Program.Log("... Saved stub IPA to '{0}'", Path.GetFullPath(IpaDestFilename));
					}
					break;

				case "stagemacfiles":
					Program.Log("Copying all staged files to Mac " + MacName + " ...");
					FileOperations.BatchUploadFolder(MacName, Config.PCStagingRootDir, MacStagingRootDir, false);
					FileOperations.BatchUploadFolder(MacName, Config.PCXcodeStagingDir, MacXcodeStagingDir, false);
					break;

				case "exportcertificate":
					CompileTime.ExportCertificate();
					break;

				default:
					return false;
			}

			return true;
		}
	}
}
