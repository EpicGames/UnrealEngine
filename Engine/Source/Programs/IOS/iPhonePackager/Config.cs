/*
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Diagnostics;
using System.Text.RegularExpressions;

namespace iPhonePackager
{
	internal class Config
	{
		/// <summary>
		/// The display name in title bars, popups, etc...
		/// </summary>
		public static string AppDisplayName = "Unreal iOS Configuration";

		public static string RootRelativePath = @"..\..\..\..\";

		public static string GameDirectory = "";		// "..\\..\\..\\..\\Engine\\Source\\UE4";
		public static string BinariesDirectory = "";	// "..\\..\\..\\..\\Engine\\Binaries\\IOS\\"

		/// <summary>
		/// Optional Prefix to append to .xcent and .mobileprovision files, for handling multiple certificates on same source game
		/// </summary>
		public static string SigningPrefix = "";

		public static string PCStagingRootDir = "";

		/// <summary>
		/// The local staging directory for files needed by Xcode on the Mac (on PC)
		/// </summary>
		public static string PCXcodeStagingDir = "";

		/// <summary>
		/// The staging directory from UAT that is passed into IPP (for repackaging, etc...)
		/// </summary>
		public static string RepackageStagingDirectory = "";

		/// <summary>
		/// The delta manifest for deploying files for iterative deploy
		/// </summary>
		public static string DeltaManifest = "";

		/// <summary
		/// The files to be retrieved from the device
		/// <summary>
		public static List<string> FilesForBackup = new List<string>();

		/// The project file that is passed into IPP from UAT
		/// </summary>
		public static string ProjectFile = "";

		/// <summary>
		/// The device to deploy or launch on
		/// </summary>
		public static string DeviceId = "";

		/// <summary>
		/// Determine whether to use RPC Util or not
		/// </summary>
		public static bool bUseRPCUtil = true;

		/// <summary>
		/// Optional override for the dev root on the target mac for remote builds.
		/// </summary>
		public static string OverrideDevRoot = null;

		/// <summary>
		/// Optional value specifying that we are working with tvOS
		/// </summary>
		public static string OSString = "IOS";

		/// <summary>
		/// The local build directory (on PC)
		/// </summary>
		public static string BuildDirectory
		{
			get
			{
				string StandardGameBuildDir = GameDirectory + @"\Build\IOS";
				// if the normal Build dir exists, return it, otherwise, use the program Resources directory
				return Path.GetFullPath(Directory.Exists(StandardGameBuildDir) ? 
					StandardGameBuildDir : 
					GameDirectory + @"\Resources\IOS");
			}
		}
		
		public static string XcodeGetProvisionParentDirectory = "xcodebuild -version | head -1 | sed -n 's/.* \\([0-9]*\\).*/\\1/p' | { read VERSION; if [ $VERSION -lt 16 ]; then echo \"MobileDevice\"; else echo \"Developer/Xcode/UserData\"; fi; } | xargs -L1 -I {PROVISIONPARENTDIR} ";
		
		private static string GetXcodeSpecificProvisionParentDirectory()
		{
			string res;
			if(Utilities.RunExecutableAndWait("bash", String.Format("-c \"{0} echo -n {{PROVISIONPARENTDIR}}\"", XcodeGetProvisionParentDirectory), out res) == 0)
			{
				return res;
			}
        	return "MobileDevice";
        }

		/// <summary>
		/// The shared provision library directory (on PC)
		/// </summary>
		public static string ProvisionDirectory
		{
			get {
				if (Environment.OSVersion.Platform == PlatformID.MacOSX || Environment.OSVersion.Platform == PlatformID.Unix) {
					return Environment.GetEnvironmentVariable("HOME") + "/Library/" + GetXcodeSpecificProvisionParentDirectory() + "/Provisioning Profiles/";
				} else {
					return Environment.GetFolderPath (Environment.SpecialFolder.LocalApplicationData) + "/Apple Computer/MobileDevice/Provisioning Profiles/";
				}
			}
		}

		/// <summary>
		/// The shared (Engine) build directory (on PC)
		/// </summary>
		public static string EngineBuildDirectory
		{
			get { 
				if (Environment.OSVersion.Platform == PlatformID.MacOSX || Environment.OSVersion.Platform == PlatformID.Unix) {
					return Path.GetFullPath (RootRelativePath + "Engine/Build/" + Config.OSString); 
				} else {
					return Path.GetFullPath (RootRelativePath + @"Engine\Build\" + Config.OSString); 
				}
			}
		}

		/// <summary>
		/// The local build intermediate directory (on PC)
		/// </summary>
		public static string IntermediateDirectory
		{
			get
			{
				string IntermediateGameBuildDir = GameDirectory + @"\Intermediate\" + Config.OSString;
				// if the normal Build dir exists, return it, otherwise, use the program Resources directory
				return Path.GetFullPath(Directory.Exists(IntermediateGameBuildDir) ?
					IntermediateGameBuildDir :
					BuildDirectory);
			}
		}

		static Config()
		{
			if (Environment.OSVersion.Platform == PlatformID.MacOSX || Environment.OSVersion.Platform == PlatformID.Unix) {
				RootRelativePath = "../../../../";
			}
		}

		/// <summary>
		/// The local directory cooked files are placed (on PC)
		/// </summary>
		public static string CookedDirectory
		{
			get { return Path.GetFullPath(GameDirectory + @"\Saved\Cooked\" + Config.OSString); }
		}

		/// <summary>
		/// The local directory config files are placed (on PC)
		/// </summary>
		public static string ConfigDirectory
		{
			get { return Path.GetFullPath(GameDirectory + @"\Saved\Config"); }
		}

		/// <summary>
		/// The engine config files are placed (on PC)
		/// </summary>
		public static string DefaultConfigDirectory
		{
			get { return Path.GetFullPath(RootRelativePath + @"Engine\Config"); }
		}

		/// <summary>
		/// The engine directory
		/// </summary>
		public static string EngineDirectory
		{
			get { return Path.GetFullPath(RootRelativePath + @"Engine"); }
		}

		/// <summary>
		/// The local directory that a payload (GameName.app) is assembled into before being copied to the Mac (on PC)
		/// </summary>
		//@TODO: Deprecate this directory
		public static string PayloadDirectory
		{
			get { return Path.GetFullPath(PCStagingRootDir + @"\Payload\" + GetTargetName() + Program.Architecture + ".app"); }
		}

		/// <summary>
		/// The local directory that a payload (GameName.app) is assembled into before being copied to the Mac (on PC)
		/// </summary>
		public static string PayloadCookedDirectory
		{
			get { return Path.GetFullPath(PayloadDirectory + @"\cookeddata"); }
		}

		/// <summary>
		/// The local directory that a payload (GameName.app) is assembled into before being copied to the Mac (on PC)
		/// </summary>
		public static string PayloadRootDirectory
		{
			get { return Path.GetFullPath(PCStagingRootDir + @"\Payload"); }
		}

		/// <summary>
		/// Returns the filename for the IPA (no path, just filename)
		/// </summary>
		public static string IPAFilenameOnMac
		{
			get
			{
				if (Program.GameConfiguration == "Development")
				{
					return SigningPrefix + GetTargetName() + Program.Architecture + ".ipa";
				}
				else
				{
					return SigningPrefix + GetTargetName() + "-" + Config.OSString + "-" + Program.GameConfiguration + Program.Architecture + ".ipa";
				}
			}
		}

		/// <summary>
		/// Returns the name of the file containing user overrides that will be applied when packaging on PC
		/// </summary>
		public static string GetPlistOverrideFilename()
		{
			return GetPlistOverrideFilename(false);
		}

		public static string GetPlistOverrideFilename(bool bWantDistributionOverlay)
		{
			string Prefix = "";
			if (bWantDistributionOverlay)
			{
				Prefix = "Distro_";
			}

			return Path.Combine(BuildDirectory, Prefix + Program.GameName + "Overrides.plist");
		}

		/// <summary>
		/// Returns the full path for either the stub or final IPA on the PC
		/// </summary>
		public static string GetIPAPath(string FileSuffix)
		{
			// Quash the default Epic_ so that stubs for Launcher installers get named correctly and can be used
			string FilePrefix = (SigningPrefix == "Epic_") ? "" : SigningPrefix;

			string Filename;

			if (Program.GameConfiguration == "Development")
			{
				Filename = Path.Combine(Config.BinariesDirectory, FilePrefix + GetTargetName() + Program.Architecture + FileSuffix);
			}
			else
			{
				Filename = Path.Combine(Config.BinariesDirectory, FilePrefix + GetTargetName() + "-" + Config.OSString + "-" + Program.GameConfiguration + Program.Architecture + FileSuffix);
			}

			return Filename;
		}

		public static string RemapIPAPath(string FileSuffix)
		{
			// Quash the default Epic_ so that stubs for Launcher installers get named correctly and can be used
			string FilePrefix = (SigningPrefix == "Epic_") ? "" : SigningPrefix;

			string Filename;

			string BinariesDir = BinariesDirectory;
			string ExeName = Program.GameName;
			if (!String.IsNullOrEmpty(ProjectFile))
			{
				BinariesDir = Path.Combine(Path.GetDirectoryName(ProjectFile), "Binaries", Config.OSString);
				ExeName = Path.GetFileNameWithoutExtension(ProjectFile);
			}
			if (!bIsCodeBasedProject)
			{
				ExeName = (Program.IsClient ? "UnrealClient" : "UnrealGame");
			}
			else if(Program.IsClient)
			{
				ExeName += "Client";
			}

			if (Program.GameConfiguration == "Development")
			{
				Filename = Path.Combine(BinariesDir, FilePrefix + ExeName + Program.Architecture + FileSuffix);
			}
			else
			{
				Filename = Path.Combine(BinariesDir, FilePrefix + ExeName + "-" + Config.OSString + "-" + Program.GameConfiguration + Program.Architecture + FileSuffix);
			}

			// ensure the directory exists
			
			return Filename;
		}


		/// <summary>
		/// Returns target name, either the GameName + optional Client or specified by -targetname
		/// </summary>
		public static string GetTargetName()
		{
			return TargetName != "" ? TargetName : Program.GameName + (Program.IsClient ? "Client" : "");
		}

		/// <summary>
		/// Returns the full path for the stub IPA, following the signing prefix resolution rules
		/// </summary>
		public static string GetIPAPathForReading(string FileSuffix)
		{
			if (Program.GameConfiguration == "Development")
			{
				return FileOperations.FindPrefixedFile(Config.BinariesDirectory, GetTargetName() + Program.Architecture + FileSuffix);
			}
			else
			{
				return FileOperations.FindPrefixedFile(Config.BinariesDirectory, GetTargetName() + "-" + Config.OSString + "-" + Program.GameConfiguration + Program.Architecture + FileSuffix);
			}
		}

		/// <summary>
		/// Whether or not to output extra information (like every file copy and date/time stamp)
		/// </summary>
		public static bool bVerbose = true;

		/// <summary>
		/// Whether or not to output extra information in code signing
		/// </summary>
		public static bool bCodeSignVerbose = false;

		/// <summary>
		/// Whether or not non-critical files will be packaged (critical == required for signing or .app validation, the app may still fail to
		/// run with only 'critical' files present).  Also affects the name and location of the IPA back on the PC
		/// </summary>
		public static bool bCreateStubSet = false;

		/// <summary>
		/// Is this a distribution packaging build?  Controls a number of aspects of packing (which signing prefix and provisioning profile to use, etc...)
		/// </summary>
		public static bool bForDistribution = false;

		/// <summary>
		/// Is this a c++ code based project?  So far used on repackagefromstage to properly choose the .ipa name
		/// </summary>
		public static bool bIsCodeBasedProject = false;

		/// <summary>
		/// Is this going to try automatic signing?
		/// </summary>
		public static bool bAutomaticSigning = false;

		/// <summary>
		/// Whether or not to strip symbols (they will always be stripped when packaging for distribution)
		/// </summary>
		public static bool bForceStripSymbols = false;

		/// <summary>
		/// Do a code signing update when repackaging?
		/// </summary>
		public static bool bPerformResignWhenRepackaging = false;

		/// <summary>
		/// Whether the cooked data will be cooked on the fly or was already cooked by the books.
		/// </summary>
		public static bool bCookOnTheFly = false;

		/// <summary>
		/// Whether the install should be performed on a provision
		/// </summary>
		public static bool bProvision = false;

		/// <summary>
		/// provision to be installed
		/// </summary>
		public static string Provision = "";

        /// <summary>
		/// provision to be installed
		/// </summary>
		public static string ProvisionUUID = "";

        /// <summary>
		/// provision to extract certificate for
		/// </summary>
		public static string ProvisionFile;

		/// <summary>
		/// target name specificied by the command line, if not set GameName + optional Client will be used
		/// </summary>
		public static string TargetName = "";

		/// <summary>
		/// IOS Team ID to be used for automatic signing
		/// </summary>
		public static string TeamID = "";

		/// <summary>
		/// Whether the install should be performed on a certificate
		/// </summary>
		public static bool bCert = false;

		/// <summary>
		/// Certificate to be installed
		/// </summary>
		public static string Certificate = "";

		/// <summary>
		/// Certificate to be output
		/// </summary>
		public static string OutputCertificate = null;

		/// <summary>
		/// An override server Mac name
		/// </summary>
		public static string OverrideMacName = null;

		/// <summary>
		/// A name to use for the bundle indentifier and display name when resigning
		/// </summary>
		public static string OverrideBundleName = null;

		/// <summary>
		/// Whether to use None or Best for the compression setting when repackaging IPAs (int version of Ionic.Zlib.CompressionLevel)
		/// By making this an int, we are able to delay load the Ionic assembly from a different path (which is required)
		/// </summary>
		public static int RecompressionSetting = 0; //Ionic.Zlib.CompressionLevel.None;

		/// <summary>
		/// Returns the application directory inside the zip
		/// </summary>
		public static string AppDirectoryInZIP
		{
			get { return _AppDirectoryInZIP != null ? _AppDirectoryInZIP : String.Format("Payload/{0}{1}.app", GetTargetName(), Program.Architecture); }
			set { _AppDirectoryInZIP = value.TrimEnd("/".ToCharArray()); }
		}
		private static string _AppDirectoryInZIP = null;

		/// <summary>
		/// If the requirements blob is present in the existing executable when code signing, should it be carried forward
		/// (true) or should a dummy requirements blob be created with no actual requirements (false)
		/// </summary>
		public static bool bMaintainExistingRequirementsWhenCodeSigning = false;

		/// <summary>
		/// The code signing identity to use when signing via RPC
		/// </summary>
		public static string CodeSigningIdentity;

		/// <summary>
		/// The minimum OS version
		/// </summary>
		public static string MinOSVersion;

		/// <summary>
		/// Create an iterative IPA (i.e. a stub only with Icons and Splash images)
		/// </summary>
		public static bool bIterate = false;

		/// <summary>
		/// Returns a path to the place to back up documents from a device
		/// </summary>
		/// <returns></returns>
		public static string GetRootBackedUpDocumentsDirectory()
		{
			return Path.GetFullPath(Path.Combine(GameDirectory + @"\" + Config.OSString + "_Backups"));
		}

		public static bool Initialize(string InitialCurrentDirectory, string GamePath)
		{
			bool bIsEpicInternal = File.Exists(@"..\..\EpicInternal.txt");


			// if the path is a directory (relative to where the game was launched from), then get the absolute directory
			string FullGamePath = Path.GetFullPath(Path.Combine(InitialCurrentDirectory, GamePath));
			string OrigGamePath = GamePath;

			if (Directory.Exists(FullGamePath))
			{
				GameDirectory = FullGamePath;
			}
			// is it a file? if so, just use the file's directory
			else if (File.Exists(FullGamePath))
			{
				GameDirectory = Path.GetDirectoryName(FullGamePath);
			}
			// else we assume old school game name and look for it
			else
			{
				if (Program.GameName == "UnrealGame") {
					GameDirectory = Path.GetFullPath (Path.Combine (Config.RootRelativePath, GamePath));
				} else {
					GameDirectory = Path.GetFullPath (Path.Combine (Config.RootRelativePath, Program.GameName));
				}
			}

			if (!Directory.Exists(GameDirectory))
			{
				Console.ForegroundColor = ConsoleColor.Red;
				Console.WriteLine();
				Console.WriteLine("Unable to find a game or program {0}. You may need to specify a path to the program", OrigGamePath);
				Console.ResetColor();
				return false ;
			}

			// special case handling for anything inside Engine/Source, it will go to the Engine/Binaries directory, not the Game's binaries directory
			if (OrigGamePath.Replace("\\", "/").Contains("Engine/Source"))
			{
				BinariesDirectory = Path.Combine(RootRelativePath, @"Engine\Binaries\" + Config.OSString + @"\");
			}
			else if (!OrigGamePath.Contains(@"Binaries\" + Config.OSString))
			{
				// no sense in adding Binaries\IOS when it's already there. This is a special case to handle packaging UnrealLaunchDaemon from the command line.
				BinariesDirectory = Path.Combine(GameDirectory, @"Binaries\" + Config.OSString + @"\");
			}
			else
			{
				BinariesDirectory = GameDirectory;
			}

			// Root directory on PC for staging files to copy to Mac
			Config.PCStagingRootDir = String.Format(@"{0}-Deploy\{1}\{2}{3}\",
				IntermediateDirectory,
				GetTargetName(),
				Program.GameConfiguration,
				Program.Architecture);

			// make a directory for the shared XcodeSupportFiles directory
			Config.PCXcodeStagingDir = Path.Combine(Config.PCStagingRootDir, @"..\XcodeSupportFiles");

			// Code signing identity
			// Rules:
			//   An environment variable wins if set
			//   Otherwise for internal development builds, an internal identity is used
			//   Otherwise, developer or distribution are used
			// Distro builds won't succeed on a machine with multiple distro certs installed unless the environment variable is set.
			Config.CodeSigningIdentity = Config.bForDistribution ? "iPhone Distribution" : "iPhone Developer";


			if (Config.bForDistribution)
			{
				Config.CodeSigningIdentity = Utilities.GetEnvironmentVariable("ue.IOSDistributionSigningIdentity", Config.CodeSigningIdentity);
			}
			else
			{
				Config.CodeSigningIdentity = Utilities.GetEnvironmentVariable("ue.IOSDeveloperSigningIdentity", Config.CodeSigningIdentity);
			}

			// Remember to also change the default min version in UBT (iPhoneToolChain.cs)
			Config.MinOSVersion = Utilities.GetEnvironmentVariable("ue3.iPhone_MinOSVersion", "12.0");

			// look for the signing prefix environment variable
			string DefaultPrefix = "";
			if (bIsEpicInternal)
			{
				// default Epic to "Epic_" prefix
				DefaultPrefix = "Epic_";
			}
			
			if (Config.bForDistribution)
			{
				DefaultPrefix = "Distro_";
			}
			Config.SigningPrefix = Utilities.GetEnvironmentVariable("ue.IOSSigningPrefix", DefaultPrefix);

			// Windows doesn't allow environment vars to be set to blank so detect "none" and treat it as such
			if (Config.SigningPrefix == "none")
			{
				Config.SigningPrefix = Config.bForDistribution ? "Distro_" : "";
			}

			CompileTime.ConfigurePaths();
			return true;
		}
	}
}
