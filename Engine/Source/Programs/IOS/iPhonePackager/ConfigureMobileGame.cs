/*
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Drawing;
using System.Linq;
using System.Text;
using System.Windows.Forms;
using System.Diagnostics;
using System.IO;
using System.Text.RegularExpressions;

namespace iPhonePackager
{
	public partial class ConfigureMobileGame : Form
	{
		public bool bSucceededConfiguring = false;

		public ConfigureMobileGame()
		{
			InitializeComponent();
		}

		// CFBundleDisplayName
		
		// CFBundleName, fewer than 16 characters long

		// CFBundleIdentifier
		// http://developer.apple.com/library/ios/#documentation/general/Reference/InfoPlistKeyReference/Articles/CoreFoundationKeys.html#//apple_ref/doc/uid/20001431-102070
		// Alphanumeric (A-Z,a-z,0-9), hyphen (-), and period (.) characters.
		// The string should also be in reverse-DNS format.

		void ReloadPList()
		{
			string Filename = Config.GetPlistOverrideFilename();
			string SourcePList = ReadOrCreate(Filename);
			Utilities.PListHelper Helper = new Utilities.PListHelper(SourcePList);
			
			string BundleID;
			string BundleName;
			string BundleDisplayName;
			if (!Helper.GetString("CFBundleIdentifier", out BundleID))
			{
				BundleID = "com.YourCompany.GameNameNoSpaces";
				Helper.SetString("CFBundleIdentifier", BundleID);
			}

			if (!Helper.GetString("CFBundleName", out BundleName))
			{
				BundleName = "MyUnrealGame";
				Helper.SetString("CFBundleName", BundleName);
			}

			if (!Helper.GetString("CFBundleDisplayName", out BundleDisplayName))
			{
				BundleDisplayName = "Unreal Game";
				Helper.SetString("CFBundleDisplayName", BundleDisplayName);
			}

			BundleIdentifierEdit.Text = BundleID;
			BundleNameEdit.Text = BundleName;
			BundleDisplayNameEdit.Text = BundleDisplayName;
		}

		private void ReloadButton_Click(object sender, EventArgs e)
		{
			ReloadPList();
		}

		private void SaveChanges()
		{
			// Development settings
			{
				// Open the existing plist override file
				string DevFilename = Config.GetPlistOverrideFilename(false);
				string SourcePList = ReadOrCreate(DevFilename);
				Utilities.PListHelper Helper = new Utilities.PListHelper(SourcePList);

				// Jam the edited values in
				if (!Helper.HasKey("UIFileSharingEnabled"))
				{
					Helper.SetValueForKey("UIFileSharingEnabled", true);
				}

				Helper.SetString("CFBundleIdentifier", BundleIdentifierEdit.Text);
				Helper.SetString("CFBundleName", BundleNameEdit.Text);
				Helper.SetString("CFBundleDisplayName", BundleDisplayNameEdit.Text);

				// Save the modified plist
				SavePList(Helper, DevFilename);
			}

			// Distribution settings
			{
				// Open the existing plist override file
				string DistFilename = Config.GetPlistOverrideFilename(true);
				string SourcePList = ReadOrCreate(DistFilename);
				Utilities.PListHelper Helper = new Utilities.PListHelper(SourcePList);

				// Jam the edited values in
				if (!Helper.HasKey("UIFileSharingEnabled"))
				{
					Helper.SetValueForKey("UIFileSharingEnabled", false);
				}

				// Save the modified plist
				SavePList(Helper, DistFilename);
			}
		}

		private void EditManuallyButton_Click(object sender, EventArgs e)
		{
			SaveChanges();

			string Filename = Config.GetPlistOverrideFilename();
			Process.Start("explorer.exe", String.Format("/select,\"{0}\"", Path.GetFullPath(Filename)));

			DialogResult = DialogResult.OK;
			Close();
		}

		string ReadOrCreate(string Filename)
		{
			string Data = "";
			try
			{
				Data = File.ReadAllText(Filename, Encoding.UTF8);
			}
			catch
			{
				Data = "<?xml version=\"1.0\" encoding=\"UTF-8\"?>\n" +
					"<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">\n" +
					"<plist version=\"1.0\">\n<dict>\n</dict>\n</plist>\n";
			}

			return Data;
		}

		private void ShowError(string Message)
		{
			MessageBox.Show(Message, Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
		}

		private void GenerateResignFileButton_Click(object sender, EventArgs e)
		{
			// Validate the edited values
			Regex AcceptableID = new Regex("[^a-zA-Z0-9.\\-]");
			if (AcceptableID.IsMatch(BundleIdentifierEdit.Text))
			{
				MessageBox.Show("Error: Bundle Identifier must contain only characters in the range [a-z] [A-Z] [0-9] . and -");
				return;
			}

			if (BundleIdentifierEdit.Text.Length < 1)
			{
				ShowError("Error: Bundle Identifier must be at least 1 character long");
				return;
			}

			if ((BundleNameEdit.Text.Length < 1) || (BundleNameEdit.Text.Length >= 16))
			{
				ShowError("Error: Bundle Name must be between 1 and 15 characters long");
				return;
			}

			if (BundleDisplayNameEdit.Text.Length < 1)
			{
				BundleDisplayNameEdit.Text = BundleNameEdit.Text;
			}

			SaveChanges();

			DialogResult = DialogResult.OK;
			Close();
		}

		private void SavePList(Utilities.PListHelper Helper, string Filename)
		{
			FileInfo DestInfo = new FileInfo(Filename);
			if (DestInfo.Exists && DestInfo.IsReadOnly)
			{
				DestInfo.IsReadOnly = false;
			}
			Helper.SaveToFile(Filename);
		}

		private void ConfigureMobileGame_Load(object sender, EventArgs e)
		{
			ReloadPList();
			Text = Config.AppDisplayName + ": Customize Info.plist";
		}

		private void ShowHyperlink(object sender, LinkLabelLinkClickedEventArgs e)
		{
			string Target = (sender as LinkLabel).Tag as string;
			ProcessStartInfo PSI = new ProcessStartInfo(Target);
			Process.Start(PSI);
		}
	}
}
