/**
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Net;
using System.Runtime.InteropServices;
using System.Text;
using System.Linq;
using System.Threading;
using Ionic.Zip;
using Ionic.Zlib;

namespace iPhonePackager
{
	/**
	 * Operations performed done at cook time - there should be no calls to the Mac here
	 */
	public class CookTime
	{
		/// <summary>
		/// List of files being inserted or updated in the Zip
		/// </summary>
		static private HashSet<string> FilesBeingModifiedToPrintOut = new HashSet<string>();

		/**
		 * Create and open a work IPA file
		 */
		static private ZipFile SetupWorkIPA()
		{
			string ReferenceZipPath = Config.GetIPAPathForReading(".stub");
			string WorkIPA = Config.GetIPAPath(".ipa");
			return CreateWorkingIPA(ReferenceZipPath, WorkIPA);
		}

		/// <summary>
		/// Creates a copy of a source IPA to a working path and opens it up as a Zip for further modifications
		/// </summary>
		static private ZipFile CreateWorkingIPA(string SourceIPAPath, string WorkIPAPath)
		{
			FileInfo ReferenceInfo = new FileInfo(SourceIPAPath);
			if (!ReferenceInfo.Exists)
			{
				Program.Error(String.Format("Failed to find stub IPA '{0}'", SourceIPAPath));
				return null;
			}
			else
			{
				Program.Log(String.Format("Loaded stub IPA from '{0}' ...", SourceIPAPath));
			}

			if (Program.GameName == "UnrealGame")
			{
				WorkIPAPath = Config.RemapIPAPath(".ipa");
			}

			// Make sure there are no stale working copies around
			FileOperations.DeleteFile(WorkIPAPath);

			// Create a working copy of the IPA
			FileOperations.CopyRequiredFile(SourceIPAPath, WorkIPAPath);

			// Open up the zip file
			ZipFile Stub = ZipFile.Read(WorkIPAPath);

			// modern xcode uses Foo-IOS-Config style .app directories, so don't assume GameName, pull it out of of the zip
			Config.AppDirectoryInZIP = FileOperations.ZipFileSystem.FindRootPathInIPA(Stub);

			// Do a few quick spot checks to catch problems that may have occurred earlier
			bool bHasCodeSignature = Stub[Config.AppDirectoryInZIP + "/_CodeSignature/CodeResources"] != null;
			if (!bHasCodeSignature)
			{
				Program.Error("Stub IPA does not appear to be signed correctly (missing _CodeSignature/CodeResources)");
				Program.ReturnCode = (int)ErrorCodes.Error_StubNotSignedCorrectly;
			}

			// Set encoding to support unicode filenames
			Stub.AlternateEncodingUsage = ZipOption.Always;
			Stub.AlternateEncoding = Encoding.UTF8;

			return Stub;
		}

		/// <summary>
		/// Extracts Info.plist from a Zip
		/// </summary>
		static private string ExtractEmbeddedPList(ZipFile Zip)
		{
			// Extract the existing Info.plist
			string PListPathInZIP = String.Format("{0}/Info.plist", Config.AppDirectoryInZIP);
			if (Zip[PListPathInZIP] == null)
			{
				Program.Error("Failed to find Info.plist in IPA (cannot update plist version)");
				Program.ReturnCode = (int)ErrorCodes.Error_IPAMissingInfoPList;
				return "";
			}
			else
			{
				// Extract the original into a temporary directory
				string TemporaryName = Path.GetTempFileName();
				FileStream OldFile = File.OpenWrite(TemporaryName);
				Zip[PListPathInZIP].Extract(OldFile);
				OldFile.Close();

				// Read the text and delete the temporary copy
				string PListSource = File.ReadAllText(TemporaryName, Encoding.UTF8);
				File.Delete(TemporaryName);

				return PListSource;
			}
		}

		static private void CopyEngineOrGameFiles(string Subdirectory, string Filter)
		{
			// build the matching paths
			string EngineDir = Path.Combine(Config.EngineBuildDirectory + Subdirectory);
			string GameDir = Path.Combine(Config.BuildDirectory + Subdirectory);

			// find the files in the engine directory
			string[] EngineFiles = Directory.GetFiles(EngineDir, Filter);

			if (Directory.Exists(GameDir))
			{
				string[] GameFiles = Directory.GetFiles(GameDir, Filter);

				// copy all files from game
				foreach (string GameFilename in GameFiles)
				{
					string DestFilename = Path.Combine(Config.PayloadDirectory, Path.GetFileName(GameFilename));
					// copy the game verison instead of engine version
					FileOperations.CopyRequiredFile(GameFilename, DestFilename);
				}
			}

			// look for matching engine files that weren't in game
			foreach (string EngineFilename in EngineFiles)
			{
				string GameFilename = Path.Combine(GameDir, Path.GetFileName(EngineFilename));
				string DestFilename = Path.Combine(Config.PayloadDirectory, Path.GetFileName(EngineFilename));
				if (!File.Exists(GameFilename))
				{
					// copy the game verison instead of engine version
					FileOperations.CopyRequiredFile(EngineFilename, DestFilename);
				}
			}
		}

		static public void CopySignedFiles()
		{
			string TargetName = Config.GetTargetName();
			string NameDecoration;
			if (Program.GameConfiguration == "Development")
			{
				NameDecoration = Program.Architecture;
			}
			else
			{
				NameDecoration = "-" + Config.OSString + "-" + Program.GameConfiguration + Program.Architecture;
			}

			// Copy and un-decorate the binary name
			FileOperations.CopyFiles(Config.BinariesDirectory, Config.PayloadDirectory, "<PAYLOADDIR>", TargetName + NameDecoration, null);
			FileOperations.RenameFile(Config.PayloadDirectory, TargetName + NameDecoration, TargetName);

			FileOperations.CopyNonEssentialFile(
				Path.Combine(Config.BinariesDirectory, TargetName + NameDecoration + ".app.dSYM.zip"),
				Path.Combine(Config.PCStagingRootDir, TargetName + NameDecoration + ".app.dSYM.zip.datecheck")
			);
		}

		/**
		 * Callback for setting progress when saving zip file
		 */
		static private void UpdateSaveProgress(object Sender, SaveProgressEventArgs Event)
		{
			if (Event.EventType == ZipProgressEventType.Saving_BeforeWriteEntry)
			{
				if (FilesBeingModifiedToPrintOut.Contains(Event.CurrentEntry.FileName))
				{
					Program.Log(" ... Packaging '{0}'", Event.CurrentEntry.FileName);
				}
			}
		}

		/// <summary>
		/// Updates the version string and then applies the settings in the user overrides plist
		/// </summary>
		/// <param name="Info"></param>
		public static void UpdateVersion(Utilities.PListHelper Info)
		{
			// Update the minor version number if the current one is older than the version tracker file
			// Assuming that the version will be set explicitly in the overrides file for distribution
			VersionUtilities.UpdateMinorVersion(Info);

			// Mark the type of build (development or distribution)
			Info.SetString("EpicPackagingMode", Config.bForDistribution ? "Distribution" : "Development");
		}

		/** 
		 * Using the stub IPA previously compiled on the Mac, create a new IPA with assets
		 */
		static public void RepackageIPAFromStub()
		{
			if (string.IsNullOrEmpty(Config.RepackageStagingDirectory) || !Directory.Exists(Config.RepackageStagingDirectory))
			{
				Program.Error("Directory specified with -stagedir could not be found!");
				return;
			}


			DateTime StartTime = DateTime.Now;
			CodeSignatureBuilder CodeSigner = null;

			// Clean the staging directory
			Program.ExecuteCommand("Clean", null);

			// Create a copy of the IPA so as to not trash the original
			ZipFile Zip = SetupWorkIPA();
			if (Zip == null)
			{
				return;
			}

			FileOperations.ZipFileSystem FileSystem = new FileOperations.ZipFileSystem(Zip);
			// Check for a staged plist that needs to be merged into the main one
			{
				// Determine if there is a staged one we should try to use instead
				string PossiblePList = Path.Combine(Config.RepackageStagingDirectory, "Info.plist");
				if (File.Exists(PossiblePList))
				{
					if (Config.bPerformResignWhenRepackaging)
					{
						Program.Log("Found Info.plist ({0}) in stage, which will be merged in with stub plist contents", PossiblePList);

						// Merge the two plists, using the staged one as the authority when they conflict
						byte[] StagePListBytes = File.ReadAllBytes(PossiblePList);
						string StageInfoString = Encoding.UTF8.GetString(StagePListBytes);

						byte[] StubPListBytes = FileSystem.ReadAllBytes("Info.plist");
						Utilities.PListHelper StubInfo = new Utilities.PListHelper(Encoding.UTF8.GetString(StubPListBytes));

						// don't overwrite the Exe name in the plist, because we may be merging in Development staged data
						// with a Shipping binary (or whatever combination), and modern uses different-named executables for each config
						StubInfo.MergePlistIn(StageInfoString, new HashSet<string> { "CFBundleExecutable" } );

						// Write it back to the cloned stub, where it will be used for all subsequent actions
						byte[] MergedPListBytes = Encoding.UTF8.GetBytes(StubInfo.SaveToString());
						FileSystem.WriteAllBytes("Info.plist", MergedPListBytes);
					}
					else
					{
						Program.Warning("Found Info.plist ({0}) in stage that will be ignored; IPP cannot combine it with the stub plist since -sign was not specified", PossiblePList);
					}
				}
			}

			// Get the name of the executable file
			string CFBundleExecutable;
			{
				// Load the .plist from the stub
				byte[] RawInfoPList = FileSystem.ReadAllBytes("Info.plist");

				Utilities.PListHelper Info = new Utilities.PListHelper(Encoding.UTF8.GetString(RawInfoPList));

				// Get the name of the executable file
				if (!Info.GetString("CFBundleExecutable", out CFBundleExecutable))
				{
					throw new InvalidDataException("Info.plist must contain the key CFBundleExecutable");
				}
			}

			// Tell the file system about the executable file name so that we can set correct attributes on
			// the file when zipping it up
			FileSystem.ExecutableFileName = CFBundleExecutable;

			// Prepare for signing if requested
			if (Config.bPerformResignWhenRepackaging)
			{
				// Start the resign process (load the mobileprovision and info.plist, find the cert, etc...)
				CodeSigner = new CodeSignatureBuilder();
				CodeSigner.FileSystem = FileSystem;

				CodeSigner.PrepareForSigning();

				// Merge in any user overrides that exist
				UpdateVersion(CodeSigner.Info);
			}

			// Empty the current staging directory
			FileOperations.DeleteDirectory(new DirectoryInfo(Config.PCStagingRootDir));

			// we will zip files in the pre-staged payload dir
			string ZipSourceDir = Config.RepackageStagingDirectory;

			// Save the zip
			Program.Log("Saving IPA ...");

			FilesBeingModifiedToPrintOut.Clear();
			Zip.SaveProgress += UpdateSaveProgress;

			Zip.CompressionLevel = (Ionic.Zlib.CompressionLevel)Config.RecompressionSetting;

			// Add all of the payload files, replacing existing files in the stub IPA if necessary (should only occur for icons)
			{
				string SourceDir = Path.GetFullPath(ZipSourceDir);
				string[] PayloadFiles = Directory.GetFiles(SourceDir, "*.*", Config.bIterate ? SearchOption.TopDirectoryOnly : SearchOption.AllDirectories);

				foreach (string Filename in PayloadFiles)
				{
					// Get the relative path to the file (this implementation only works because we know the files are all
					// deeper than the base dir, since they were generated from a search)
					string AbsoluteFilename = Path.GetFullPath(Filename);
					string RelativeFilename = AbsoluteFilename.Substring(SourceDir.Length + 1).Replace('\\', '/');

					string ZipAbsolutePath = String.Format("{0}/{1}",
						Config.AppDirectoryInZIP,
						RelativeFilename);

					Stream FileContents = File.OpenRead(AbsoluteFilename);
					if (FileContents.Length == 0)
					{
						// Zero-length files added by Ionic cause installation/upgrade to fail on device with error 0xE8000050
						// We store a single byte in the files as a workaround for now
						byte[] DummyContents = new byte[1];
						DummyContents[0] = 0;
						FileContents = new MemoryStream(DummyContents);
					}

					FileSystem.WriteStream(RelativeFilename, FileContents);

					if ((FileContents.Length >= 1024 * 1024) || (Config.bVerbose))
					{
						FilesBeingModifiedToPrintOut.Add(ZipAbsolutePath);
					}
				}
			}

			// Re-sign the executable if there is a signing context
			if (CodeSigner != null)
			{
				if (Config.OverrideBundleName != null)
				{
					CodeSigner.Info.SetString("CFBundleDisplayName", Config.OverrideBundleName);
					string CFBundleIdentifier;
					if (CodeSigner.Info.GetString("CFBundleIdentifier", out CFBundleIdentifier))
					{
						CodeSigner.Info.SetString("CFBundleIdentifier", CFBundleIdentifier + "_" + Config.OverrideBundleName);
					}
				}
				CodeSigner.PerformSigning();
			}

			// Stick in the iTunesArtwork PNG if available
			string iTunesArtworkPath = Path.Combine(Config.BuildDirectory, "iTunesArtwork");
			if (File.Exists(iTunesArtworkPath))
			{
				Zip.UpdateFile(iTunesArtworkPath, "");
			}

			// Save the Zip

			Program.Log("Compressing files into IPA (-compress={1}).{0}", Config.bVerbose ? "" : "  Only large files will be listed next, but other files are also being packaged.", Config.RecompressionSetting);
			FileSystem.Close();

			TimeSpan ZipLength = DateTime.Now - StartTime;

			FileInfo FinalZipInfo = new FileInfo(Zip.Name);


			Program.Log(String.Format("Finished repackaging into {2:0.00} MB IPA, written to '{0}' (took {1:0.00} s for all steps)",
				Zip.Name,
				ZipLength.TotalSeconds,
				FinalZipInfo.Length / (1024.0f * 1024.0f)));
		}

		static public bool ExecuteCookCommand(string Command, string RPCCommand)
		{
			return false;
		}
	}
}
