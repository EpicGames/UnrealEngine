/**
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Net;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
using Microsoft.Win32;
using System.Linq;
using System.Runtime.Remoting;

namespace iPhonePackager
{
	[Serializable]
	public struct ConnectedDeviceInformation
	{
		public string DeviceName;
		public string UDID;
		public string DeviceType;
		public string DeviceInterface;


	public ConnectedDeviceInformation(string DeviceName, string UDID, string DeviceType, string DeviceInterface)
		{
			this.DeviceName = DeviceName;
			this.UDID = UDID;
			this.DeviceType = DeviceType;
			this.DeviceInterface = DeviceInterface;
		}
	}

	[Serializable]
	class DeployTimeReporter
	{
		public void Log(string Line)
		{
			Program.Log(Line);
		}

		public void Error(string Line)
		{
			Program.Error(Line);
		}

		public void Warning(string Line)
		{
			Program.Warning(Line);
		}

		public void SetProgressIndex(int Progress)
		{
			Program.ProgressIndex = Progress;
		}

		public int GetTransferProgressDivider()
		{
			return (Program.BGWorker != null) ? 1000 : 25;
		}
	}

	class DeploymentHelper
	{
		public static void InstallIPAOnConnectedDevices(string IPAPath)
		{
			// Read the mobile provision to check for issues
			FileOperations.ReadOnlyZipFileSystem Zip = new FileOperations.ReadOnlyZipFileSystem(IPAPath);
			MobileProvision Provision = null;
			try
			{
				MobileProvisionParser.ParseFile(Zip.ReadAllBytes("embedded.mobileprovision"));
			}
			catch (System.Exception ex)
			{
				Program.Warning(String.Format("Couldn't find an embedded mobile provision ({0})", ex.Message));
				Provision = null;
			}
			Zip.Close();


			if (Provision != null)
			{
				var DevicesList = GetAllConnectedDevices();

				foreach (var DeviceInfo in DevicesList)
				{
					string UDID = DeviceInfo.UDID;
					string DeviceName = DeviceInfo.DeviceName;

					// Check the IPA's mobile provision against the connected device to make sure this device is authorized
					// We'll still try installing anyways, but this message is more friendly than the failure we got back from MobileDeviceInterface when we used it
					if (UDID != String.Empty)
					{
						if (!Provision.ContainsUDID(UDID))
						{
							Program.Warning(String.Format("Embedded provision in IPA does not include the UDID {0} of device '{1}'.  The installation is likely to fail.", UDID, DeviceName));
						}
					}
					else
					{
						Program.Warning(String.Format("Unable to query device for UDID, and therefore unable to verify IPA embedded mobile provision contains this device."));
					}
				}
			}
			InstallIPAOnDevices(IPAPath);
		}

		public static bool ExecuteDeployCommand(string Command, string GamePath, string RPCCommand)
		{
			switch (Command.ToLowerInvariant())
			{
				case "backup":
					{
						string ApplicationIdentifier = RPCCommand;
						if (ApplicationIdentifier == null)
						{
							ApplicationIdentifier = Utilities.GetStringFromPList("CFBundleIdentifier");
						}

						if (Config.FilesForBackup.Count > 0)
						{
							if (!BackupFiles(ApplicationIdentifier, Config.FilesForBackup.ToArray()))
							{
								Program.Error("Failed to transfer manifest file from device to PC");
								Program.ReturnCode = (int)ErrorCodes.Error_DeviceBackupFailed;
							}
						}
						else if (!DeploymentHelper.BackupDocumentsDirectory(ApplicationIdentifier, Config.GetRootBackedUpDocumentsDirectory()))
						{
							Program.Error("Failed to transfer documents directory from device to PC");
							Program.ReturnCode = (int)ErrorCodes.Error_DeviceBackupFailed;
						}
					}
					break;
				case "uninstall":
					{
						string ApplicationIdentifier = RPCCommand;
						if (ApplicationIdentifier == null)
						{
							ApplicationIdentifier = Utilities.GetStringFromPList("CFBundleIdentifier");
						}

						if (!UninstallIPAOnDevices(ApplicationIdentifier))
						{
							Program.Error("Failed to uninstall IPA on device");
							Program.ReturnCode = (int)ErrorCodes.Error_AppUninstallFailed;
						}
					}
					break;
				case "deploy":
				case "install":
					{
						string IPAPath = GamePath;
						string AdditionalCommandline = Program.AdditionalCommandline;

						if (!String.IsNullOrEmpty(AdditionalCommandline) && !Config.bIterate)
						{
							// Read the mobile provision to check for issues
							FileOperations.ReadOnlyZipFileSystem Zip = new FileOperations.ReadOnlyZipFileSystem(IPAPath);
							try
							{
								// Compare the commandline embedded to prevent us from any unnecessary writing.
								byte[] CommandlineBytes = Zip.ReadAllBytes("uecommandline.txt");
								string ExistingCommandline = Encoding.UTF8.GetString(CommandlineBytes, 0, CommandlineBytes.Length);
								if (ExistingCommandline != AdditionalCommandline)
								{
									// Ensure we have a temp dir to stage our temporary ipa
									if( !Directory.Exists( Config.PCStagingRootDir ) )
									{
										Directory.CreateDirectory(Config.PCStagingRootDir);
									}

									string TmpFilePath = Path.Combine(Path.GetDirectoryName(Config.PCStagingRootDir), Path.GetFileNameWithoutExtension(IPAPath) + ".tmp.ipa");
									if( File.Exists( TmpFilePath ) )
									{
										File.Delete(TmpFilePath);
									}

									File.Copy(IPAPath, TmpFilePath);
								
									// Get the project name:
									string ProjectFile = ExistingCommandline.Split(' ').FirstOrDefault();

									// Write out the new commandline.
									FileOperations.ZipFileSystem WritableZip = new FileOperations.ZipFileSystem(TmpFilePath);
									byte[] NewCommandline = Encoding.UTF8.GetBytes(ProjectFile + " " + AdditionalCommandline);
									WritableZip.WriteAllBytes("uecommandline.txt", NewCommandline);

									// We need to residn the application after the commandline file has changed.
									CodeSignatureBuilder CodeSigner = new CodeSignatureBuilder();
									CodeSigner.FileSystem = WritableZip;

									CodeSigner.PrepareForSigning();
									CodeSigner.PerformSigning();

									WritableZip.Close();

									// Set the deploying ipa path to our new ipa
									IPAPath = TmpFilePath;
								}
							}
							catch (System.Exception ex)
							{
								Program.Warning(String.Format("Failed to override the commandline.txt file: ({0})", ex.Message));
							}
							Zip.Close();
						}

						if (Config.bIterate)
						{
							string ApplicationIdentifier = RPCCommand;
							if (String.IsNullOrEmpty(ApplicationIdentifier))
							{
								ApplicationIdentifier = Utilities.GetStringFromPList("CFBundleIdentifier");
							}

							if (!DeploymentHelper.PushFiles(ApplicationIdentifier, Config.DeltaManifest))
							{
								Program.Error("Failed to install Files on device");
								Program.ReturnCode = (int)ErrorCodes.Error_FilesInstallFailed;
							}
						}
						else if (File.Exists(IPAPath))
						{
							if (!InstallIPAOnDevices(IPAPath))
							{
								Program.Error("Failed to install IPA on device");
								Program.ReturnCode = (int)ErrorCodes.Error_AppInstallFailed;
							}
						}
						else
						{
							Program.Error(String.Format("Failed to find IPA file: '{0}'", IPAPath));
							Program.ReturnCode = (int)ErrorCodes.Error_AppNotFound;
						}
					}
					break;

				default:
					return false;
			}

			return true;
		}

		static DeployTimeReporter Reporter = new DeployTimeReporter();
		static string ExecuteLibimobileProcess(string ProcessName, string ProcessArguments = "", bool bTreatOutputAsUTF8 = true)
		{
			string ExePath = Directory.GetCurrentDirectory() + "/../../../Extras/ThirdPartyNotUE/libimobiledevice/";

			if (Environment.OSVersion.Platform == PlatformID.Win32NT)
			{
				ExePath += "x64/";
				ProcessName += ".exe";
			}
			else if (Environment.OSVersion.Platform == PlatformID.MacOSX)
			{
				ExePath += "Mac/";
			}
			else
			{
				Program.LogVerbose("Unsupported Platform.");
				return "";
			}

			ProcessStartInfo StartInfo = new ProcessStartInfo(ExePath + ProcessName , ProcessArguments);
			StartInfo.UseShellExecute = false;
			StartInfo.RedirectStandardOutput = true;
			StartInfo.RedirectStandardError = true;
			StartInfo.CreateNoWindow = true;
			if(bTreatOutputAsUTF8)
			{
				StartInfo.StandardOutputEncoding = Encoding.UTF8;
				StartInfo.StandardErrorEncoding = Encoding.UTF8;
			}

			string FullOutput = "";
			string ErrorOutput = "";
			using (Process LocalProcess = Process.Start(StartInfo))
			{
				StreamReader OutputReader = LocalProcess.StandardOutput;
				FullOutput = OutputReader.ReadToEnd().Trim();

				StreamReader ErrorReader = LocalProcess.StandardError;
				ErrorOutput = ErrorReader.ReadToEnd().Trim();

				if (FullOutput.Length > 0)
				{
					Program.LogVerbose(FullOutput);
				}

				if (ErrorOutput.Length > 0)
				{
					Program.LogVerbose(ErrorOutput);
				}


				LocalProcess.WaitForExit();
			}

			if (ErrorOutput.Length > 0)
			{
				if (FullOutput.Length > 0)
				{
					FullOutput += Environment.NewLine;
				}
				FullOutput += ErrorOutput;
			}
			return FullOutput;
		}
		public static  bool PushFiles(string BundleIdentifier, string ManifestFile)
		{
			string FullOutput = "";
			var DeviceList = GetAllConnectedDevices();

			foreach (var Device in DeviceList)
			{
				string IdeviceFSArgs = "";
				if (Device.DeviceInterface == "Network")
				{
					IdeviceFSArgs = "-n";
					continue;
				}

				string[] FileList = ManifestFile.Split('\n');
				string AllCommandsToPush = "";
				foreach (string Filename in FileList)
				{
					if (!string.IsNullOrEmpty(Filename) && !string.IsNullOrWhiteSpace(Filename))
					{
						string Trimmed = Filename.Trim();
						string BaseFolder = Path.GetDirectoryName(ManifestFile);
						string SourceFilename = BaseFolder + "/" + Trimmed;
						string DestFilename = "/Library/Caches/" + Trimmed.Replace("cookeddata/", "");
						DestFilename = DestFilename.Replace('\\', '/');
						DestFilename = "\"" + DestFilename + "\"";
						string CommandToPush = "push -p \"" + SourceFilename + "\" " + DestFilename + "\n";
						AllCommandsToPush += CommandToPush;
					}
				}
				System.IO.File.WriteAllText(Directory.GetCurrentDirectory() + "/CommandsToPush.txt", AllCommandsToPush);

				IdeviceFSArgs = IdeviceFSArgs + " -u " + Device.UDID + " -b " + BundleIdentifier + " -x \"" + Directory.GetCurrentDirectory() + "/CommandsToPush.txt" + "\"";

				FullOutput = ExecuteLibimobileProcess("idevicefs", IdeviceFSArgs, false);
				File.Delete(Directory.GetCurrentDirectory() + "/CommandsToPush.txt");

				if (FullOutput != "0")
				{
					return false;
				}
			}
			return true;
		}

		static bool BackupFiles(string BundleIdentifier, string[] DestinationFiles)
		{
			string FullOutput = "";
			var DeviceList = GetAllConnectedDevices();
			foreach (var Device in DeviceList)
			{
				if (Device.DeviceInterface == "Network")
				{
					continue;
				}

				string AllCommands = "";
				foreach (var DestinationFile in DestinationFiles)
				{
					string Command = "pull " + DestinationFile;
					AllCommands += Command;
				}
				System.IO.File.WriteAllText(Directory.GetCurrentDirectory() + "/CommandsToPull.txt", AllCommands);
				string arguments = "\"" + "-u " + Device.UDID + " -b " + BundleIdentifier + " -x " + Directory.GetCurrentDirectory() + "/CommandsToPull.txt" + "\"";
				FullOutput = ExecuteLibimobileProcess("idevicefs", arguments, false);
			}
			File.Delete(Directory.GetCurrentDirectory() + "/CommandsToPull.txt");
			if (FullOutput != "0")
			{
				return false;
			}
			return true;
		}

		public static bool BackupDocumentsDirectory(string BundleIdentifier, string DestinationDocumentsDirectory)
		{
			string FullOutput = "";
			var DeviceList = GetAllConnectedDevices();
			foreach (var Device in DeviceList)
			{
				if (Device.DeviceInterface == "Network")
				{
					continue;
				}

				// Destination folder
				string TargetFolder = Path.Combine(DestinationDocumentsDirectory, Device.DeviceName);
				if (!System.IO.Directory.Exists(TargetFolder))
				{
					System.IO.Directory.CreateDirectory(TargetFolder);
				}

				// Pull /Documents
				string SourceFolder = "/Documents/ ";
				string Command = " pull " + SourceFolder + "\"" + TargetFolder + "\"";
				FullOutput = ExecuteLibimobileProcess("idevicefs", "-u " + Device.UDID + " -b " + BundleIdentifier + Command, false);
				if (FullOutput != "0")
				{
					Program.LogVerbose(FullOutput);
				}

				// Pull /Library/Caches
				SourceFolder = "/Library/Caches/ ";
				Command = " pull " + SourceFolder + "\"" + TargetFolder + "\"";
				FullOutput = ExecuteLibimobileProcess("idevicefs", "-u " + Device.UDID + " -b " + BundleIdentifier + Command, false);
				if (FullOutput != "0")
				{
					Program.LogVerbose(FullOutput);
				}
			}			
			return true;
		}

		static bool ActionOnIPAOnDevices(string args, string IPAPath)
		{
			List<ConnectedDeviceInformation> DevicesToReturn = new List<ConnectedDeviceInformation>();
			string FullOutput = ExecuteLibimobileProcess("idevice_id");
			string LibimobileDeviceArguments = "";

			var ConnectedDevicesUDIDs = FullOutput.Split(new string[] { Environment.NewLine }, StringSplitOptions.None);

			foreach (var ConnnectedUDID in ConnectedDevicesUDIDs)
			{
				string CurrentUDID = ConnnectedUDID;
				if (ConnnectedUDID.Contains("(USB)"))
				{
					CurrentUDID = ConnnectedUDID.Split(' ').First();
					LibimobileDeviceArguments = "-u " + CurrentUDID + args + "\"" + IPAPath + "\"";
					string OutputInfo = ExecuteLibimobileProcess("ideviceinstaller", LibimobileDeviceArguments);
					if (OutputInfo.Contains("DONE"))
					{
						return true;
					}
					else
					{
						Program.LogVerbose(OutputInfo);
						return false;
					}
				}
				else if (ConnnectedUDID.Contains("(Network)"))
				{
					// Network install doesn't seem to be working on Windows. USB Interface only for now.
					//	CurrentUDID = ConnnectedUDID.Split(' ').First();
					//	LibimobileDeviceArguments = "-n -u " + CurrentUDID + " -i " + IPAPath;
				}

			}
			return true;
		}

		public static bool UninstallIPAOnDevices(string IPAPath)
		{
			return ActionOnIPAOnDevices(" --uninstall ", IPAPath);
		}

		public static bool InstallIPAOnDevices(string IPAPath)
		{
			return ActionOnIPAOnDevices(" -i ", IPAPath);
		}

		public static ConnectedDeviceInformation[] GetAllConnectedDevices()
		{
			List<ConnectedDeviceInformation> DevicesToReturn = new List<ConnectedDeviceInformation>();
			string FullOutput = ExecuteLibimobileProcess("idevice_id");
			string DeviceName = "";
			string UDID = "";
			string DeviceType = "";
			string ConnectingInterface = "";

			var ConnectedDevicesUDIDs = FullOutput.Split(new string[] { Environment.NewLine }, StringSplitOptions.None);

			foreach (var ConnnectedUDID in ConnectedDevicesUDIDs)
			{
				ConnectedDeviceInformation NewConnectedDevice;
				NewConnectedDevice.UDID = "";
				String ParsedUDID = ConnnectedUDID.Split(' ').First();
				String IdeviceInfoArgs = "-u " + ParsedUDID;

				if (ConnnectedUDID.Contains("Network"))
				{
					ConnectingInterface = "Network";
					IdeviceInfoArgs = "-n " + IdeviceInfoArgs;
				}
				else
				{
					ConnectingInterface = "USB";
				}

				string OutputInfo = ExecuteLibimobileProcess("ideviceinfo", IdeviceInfoArgs);
				if (OutputInfo.Contains("not found!"))
				{
					Program.LogVerbose(OutputInfo);
				}
				else
				{
					foreach (string Line in OutputInfo.Split(Environment.NewLine.ToCharArray()))
					{
						if (Line.StartsWith("DeviceName: "))
						{
							DeviceName = Line.Substring(12);
						}
						else if (Line.StartsWith("UniqueDeviceID: "))
						{
							UDID = Line.Split(':').Last();
							UDID.Replace(" ", "");
						}
						else if (Line.StartsWith("ProductType: "))
						{
							DeviceType = Line.Split(':').Last();
							DeviceType.Replace(" ", "");
						}
					}
				}
				DevicesToReturn.Add(new ConnectedDeviceInformation(DeviceName, UDID, DeviceType, ConnectingInterface));
				DeviceName = "";
				UDID = "";
				DeviceType = "";
				ConnectingInterface = "";
			}
			return DevicesToReturn.ToArray();
		}
	}
}
