/**
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Threading;
using Ionic.Zip;
using Ionic.Zlib;
using System.Net;
using System.Net.NetworkInformation;
using System.Reflection;
using System.Runtime.Remoting;
using System.Net.Sockets;
using DotNETUtilities;

namespace iPhonePackager
{
	/**
	 * Stream proxy class which can read from a seekable stream (eg a file stream) 
	 * and prevents the original stream (file) from being closed.
	 */
	public class ProxyReadStream : Stream
	{
		public long InternalPosition;
		public Stream ParentStream;

		public ProxyReadStream(Stream InParentStream)
		{
			ParentStream = InParentStream;
			InternalPosition = 0;

			if (!ParentStream.CanSeek || !ParentStream.CanRead)
			{
				throw new NotSupportedException();
			}
		}

		public override bool CanRead
		{
			get { return true; }
		}

		public override bool CanSeek
		{
			get { return true; }
		}

		public override bool CanWrite
		{
			get { return false; }
		}

		public override long Length
		{
			get { return ParentStream.Length; }
		}

		public override long Position
		{
			get { return InternalPosition; }
			set	{ InternalPosition = value; }
		}

		public override long Seek(long offset, SeekOrigin origin)
		{
			switch (origin)
			{
			case SeekOrigin.Begin:
				InternalPosition = offset;
				break;
			case SeekOrigin.Current:
				InternalPosition += offset;
				break;
			case SeekOrigin.End:
				InternalPosition = Length + offset;
				break;
			}
			return Position;
		}

		public override void SetLength(long value)
		{
			throw new NotSupportedException();
		}

		public override void Flush()
		{
			ParentStream.Flush();
		}

		public override int Read(byte[] buffer, int offset, int count)
		{
			ParentStream.Position = InternalPosition;
			int Result = ParentStream.Read(buffer, offset, count);
			InternalPosition += Result;
			return Result;
		}

		public override void Write(byte[] buffer, int offset, int count)
		{
			throw new NotSupportedException();
		}

		protected override void Dispose(bool disposing)
		{
			// Do not dispose parent
		}

		public override void Close()
		{
			// Do not close parent
		}
	}

	public partial class FileOperations
	{
		private const int DefaultRetryCount = 5;

		private static Socket RPCSocket;

		/** Cranks up the remoting utility if needed */
		static private bool ConditionalInitRemoting(string MacName)
		{
			if (RPCCommandHelper.PingRemoteHost(MacName))
			{
				try
				{
					RPCSocket = RPCCommandHelper.ConnectToUnrealRemoteTool(MacName);
				}
				catch (Exception)
				{
					Console.WriteLine("Failed to connect to UnrealRemoteTool running on {0}", MacName);
					return false;
				}
			}
			else
			{
				return false;
			}

			return true;
		}

		/// <summary>
		/// Find a prefixable file (checks first with the prefix and then without it)
		/// </summary>
		/// <param name="BasePath">The path that the file should be in</param>
		/// <param name="FileName">The filename to check for (with and without SigningPrefix prepended to it)</param>
		/// <returns>The path to the most desirable file (may still not exist)</returns>
		static public string FindPrefixedFile(string BasePath, string FileName)
		{
			return FindPrefixedFileOrDirectory(BasePath, FileName, false);
		}

		/// <summary>
		/// Finds the first file in the directory with a given extension (if it exists).  This method is to provide some
		/// idiot-proofing (for example, .mobileprovision files, which will be expected to be saved from the website as
		/// GameName.mobileprovision or Distro_GameName.mobileprovision, but falling back to using the first one we find
		/// if the correctly named one is not found helps the user if they mess up during the process.
		/// </summary>
		static public string FindAnyFileWithExtension(string BasePath, string FileExt)
		{
			string[] FileList = Directory.GetFiles(BasePath);
			foreach (string Filename in FileList)
			{
				if (Path.GetExtension(Filename).Equals(FileExt, StringComparison.InvariantCultureIgnoreCase))
				{
					return Filename;
				}
			}

			return null;
		}

		/// <summary>
		/// Find a prefixable file or directory (checks first with the prefix and then without it)
		/// </summary>
		/// <param name="BasePath">The path that the file should be in</param>
		/// <param name="Name">The name to check for (with and without SigningPrefix prepended to it)</param>
		/// <param name="bIsDirectory">Is the desired name a directory?</param>
		/// 
		/// <returns>The path to the most desirable file (may still not exist)</returns>
		static public string FindPrefixedFileOrDirectory(string BasePath, string Name, bool bIsDirectory)
		{
			string PrefixedPath = Path.Combine(BasePath, Config.SigningPrefix + Name);

			if ((bIsDirectory && Directory.Exists(PrefixedPath)) || (!bIsDirectory && File.Exists(PrefixedPath)))
			{
				return PrefixedPath;
			}
			else
			{
				return Path.Combine(BasePath, Name);
			}
		}
		
		/**
		 * Safely delete a directory
		 */
		static public void DeleteDirectory( DirectoryInfo DirInfo )
		{
			if( DirInfo.Exists )
			{
				foreach( FileInfo Info in DirInfo.GetFiles() )
				{
					try
					{
						Info.IsReadOnly = false;
						Info.Delete();
					}
					catch( Exception Ex )
					{
						Program.Error( "Could not delete file: " + Info.FullName );
						Program.Error( "    Exception: " + Ex.Message );
						Program.ReturnCode = (int)ErrorCodes.Error_DeleteFile;
					}
				}

				foreach( DirectoryInfo SubDirInfo in DirInfo.GetDirectories() )
				{
					DeleteDirectory( SubDirInfo );
				}

				try
				{
					DirInfo.Delete();
				}
				catch( Exception Ex )
				{
					Program.Error( "Could not delete folder: " + DirInfo.FullName );
					Program.Error( "    Exception: " + Ex.Message );
					Program.ReturnCode = (int)ErrorCodes.Error_DeleteDirectory;
				}
			}
		}

		/**
		 * Safely delete a file
		 */
		static public void DeleteFile( string FileName )
		{
			FileInfo Info = new FileInfo( FileName );
			if( Info.Exists )
			{
				try
				{
					Info.IsReadOnly = false;
					Info.Delete();
				}
				catch( Exception Ex )
				{
					Program.Error( "Could not delete file: " + Info.FullName );
					Program.Error( "    Exception: " + Ex.Message );
					Program.ReturnCode = (int)ErrorCodes.Error_DeleteFile;
				}
			}
		}

		/**
		 * Safely copy a folder structure
		 */
		static private void RecursiveFolderCopy( DirectoryInfo SourceFolderInfo, DirectoryInfo DestFolderInfo, bool bLowercase )
		{
			foreach( FileInfo SourceFileInfo in SourceFolderInfo.GetFiles() )
			{
				string DestFileName = Path.Combine(DestFolderInfo.FullName, bLowercase ? SourceFileInfo.Name.ToLower() : SourceFileInfo.Name);

				CopyFile(SourceFileInfo, DestFileName, true, DefaultRetryCount);
			}

			foreach( DirectoryInfo SourceSubFolderInfo in SourceFolderInfo.GetDirectories() )
			{
				string DestFolderName = Path.Combine(DestFolderInfo.FullName, bLowercase ? SourceSubFolderInfo.Name.ToLower() : SourceSubFolderInfo.Name);
				try
				{
					Directory.CreateDirectory( DestFolderName );
				}
				catch( Exception Ex )
				{
					Program.Error( "Could not create folder: " + DestFolderName );
					Program.Error( "    Exception: " + Ex.Message );
					Program.ReturnCode = (int)ErrorCodes.Error_CreateDirectory;
				}
				RecursiveFolderCopy( SourceSubFolderInfo, new DirectoryInfo( DestFolderName ), bLowercase );
			}
		}


		static public bool CopyFolder(string SourceFolder, string DestFolder, string DisplayDestFolder, bool bDeleteTarget, bool bLowercase)
		{
			if (DisplayDestFolder == null)
			{
				DisplayDestFolder = DestFolder;
			}

			DirectoryInfo SourceFolderInfo = new DirectoryInfo(SourceFolder);
			if (!SourceFolderInfo.Exists)
			{
				Program.Error("Source folder does not exist");
				return (false);
			}

			Program.Log(" ... '" + SourceFolder + "' -> '" + DisplayDestFolder + "'");

			DirectoryInfo DestFolderInfo = new DirectoryInfo(DestFolder);
			if (bDeleteTarget && DestFolderInfo.Exists)
			{
				DeleteDirectory(DestFolderInfo);
			}

			try
			{
			Directory.CreateDirectory(DestFolderInfo.FullName);
			}
			catch( Exception Ex )
			{
				Program.Error( "Could not create folder: " + DestFolderInfo.FullName );
				Program.Error( "    Exception: " + Ex.Message );
				Program.ReturnCode = (int)ErrorCodes.Error_CreateDirectory;
			}

			RecursiveFolderCopy(SourceFolderInfo, DestFolderInfo, bLowercase);

			return (true);
		}

		static public void RecursiveBatchUploadFolder(DirectoryInfo SourceFolderInfo, string DestFolder, List<string> FilesToUpload)
		{
			// add the files to the list of files to upload
			foreach (FileInfo SourceFileInfo in SourceFolderInfo.GetFiles())
			{
				string DestFileName = DestFolder + "/" + SourceFileInfo.Name;
				FilesToUpload.Add(SourceFileInfo.FullName + ";" + DestFileName);
			}

			// recurse into subdirectories
			foreach (DirectoryInfo SourceSubFolderInfo in SourceFolderInfo.GetDirectories())
			{
				string DestFolderName = DestFolder + "/" + SourceSubFolderInfo.Name;
				RecursiveBatchUploadFolder(SourceSubFolderInfo, DestFolderName, FilesToUpload);
			}
		}

		static public bool BatchUploadFolder(string MacName, string SourceFolder, string DestFolder, bool bDeleteTarget)
		{
			if (Config.bUseRPCUtil)
			{

				if (!ConditionalInitRemoting(MacName))
				{
					return false;
				}


				if (bDeleteTarget)
				{
					// tell Mac to delete target if requested
					RPCCommandHelper.RemoveDirectory(RPCSocket, DestFolder);
				}
			}

			Program.Log(" ... '" + SourceFolder + "' -> '" + DestFolder + "'");

			DirectoryInfo SourceFolderInfo = new DirectoryInfo(SourceFolder);
			if (!SourceFolderInfo.Exists)
			{
				Program.Error("Source folder does not exist");
				return false;
			}

			// gather the files to batch upload
			List<string> FilesToUpload = new List<string>();
			RecursiveBatchUploadFolder(SourceFolderInfo, DestFolder, FilesToUpload);

			if (Config.bUseRPCUtil)
			{
				// send them off!
				RPCCommandHelper.RPCBatchUpload(RPCSocket, FilesToUpload.ToArray());
			}
			else
			{
				SSHCommandHelper.BatchUpload(MacName, FilesToUpload.ToArray());
			}

			return true;
		}

		static public bool DownloadFile(string MacName, string SourceFilename, string DestFilename)
		{
			Hashtable Results = null;

			if (Config.bUseRPCUtil)
			{
				if (!ConditionalInitRemoting(MacName))
				{
					return false;
				}

				Results = RPCCommandHelper.RPCDownload(RPCSocket, SourceFilename, DestFilename);
			}
			else
			{
				Results = SSHCommandHelper.DownloadFile(MacName, SourceFilename, DestFilename);
			}

			// success if exitcode is 0 or unspecified
			if (Results["ExitCode"] != null)
			{
				return (Int64)Results["ExitCode"] == 0;
			}
			return true;
		}

		/**
		 * Copy a single file (deleting the existing file at the destination if it exists)
		 */
		static public void CopyRequiredFile(string SourcePath, string DestPath)
		{
			Program.Log(" ... '" + SourcePath + "' -> '" + DestPath + "'");
			CopyFile(new FileInfo(SourcePath), DestPath, true, DefaultRetryCount);
		}

		/**
		 * Copy a single file (deleting the existing file at the destination if it exists)
		 */
		static public void CopyNonEssentialFile(string SourcePath, string DestPath)
		{
			CopyFile(new FileInfo(SourcePath), DestPath, false, DefaultRetryCount);
		}

		/**
		 * Copy a single file (deleting the existing file at the destination if it exists)
		 */
		static public bool CopyFile(FileInfo SourceFileInfo, string DestFileName, bool bCopyMustSucceed, int MaxRetryCount)
		{
			int CopyTryCount = 0;
			bool bFileCopiedSuccessfully = false;

			// if the file doesn't exist, but isn't needed, then just early out
			if (!bCopyMustSucceed && !SourceFileInfo.Exists)
			{
				return false;
			}

			if (Config.bVerbose)
			{
				Program.Log(String.Format("      Copy: {0} -> {1}, last modified at {2}", SourceFileInfo.FullName, DestFileName, SourceFileInfo.LastWriteTime));
			}

			do
			{
				// If this isn't the first time through, sleep a little before trying again
				if (CopyTryCount > 0)
				{
					Thread.Sleep(1000);
				}

				CopyTryCount++;

				try
				{
					// Delete the destination file if it exists
					FileInfo DestInfo = new FileInfo(DestFileName);
					if (DestInfo.Exists)
					{
						DestInfo.IsReadOnly = false;
						DestInfo.Delete();
					}

					Directory.CreateDirectory(Path.GetDirectoryName(DestFileName));

					// Copy over the new file
					SourceFileInfo.CopyTo(DestFileName);

					// Make sure the destination file is writable
					DestInfo = new FileInfo(DestFileName);
					DestInfo.IsReadOnly = false;
					DestInfo.LastWriteTime = SourceFileInfo.LastWriteTime;

					// Sanity check
					if (DestInfo.Length != SourceFileInfo.Length)
					{
						throw new Exception("Error: File copy failed (source and destination are different sizes)");
					}

					// Success!
					bFileCopiedSuccessfully = true;
				}
				catch( Exception Ex )
				{
					Program.Warning("Failed to copy file to '" + DestFileName + "'");
					Program.Warning("    Exception: " + Ex.Message + " (going to retry copy)");
				}
			}
			while (!bFileCopiedSuccessfully && (CopyTryCount < MaxRetryCount));

			if (!bFileCopiedSuccessfully)
			{
				string Msg = "Failed to copy file to " + DestFileName + " and there are no more retries!";
				if (bCopyMustSucceed)
				{
					Program.Error(Msg);
					Program.ReturnCode = (int)ErrorCodes.Error_CopyFile;
				}
				else
				{
					Program.Warning(Msg);
				}
			}

			return bFileCopiedSuccessfully;
		}

		/** 
		 * Copy a set of files
		 */
		static public bool CopyFiles( string SourceFolder, string DestFolder, string DisplayDestFolder, string FileSpec, string ExcludeExtension )
		{
			if( DisplayDestFolder == null )
			{
				DisplayDestFolder = DestFolder;
			}

			Program.Log( " ... '" + SourceFolder + "\\" + FileSpec + "' -> '" + DisplayDestFolder + "'" );

			// Ensure the source folder exists - fail otherwise
			DirectoryInfo SourceFolderInfo = new DirectoryInfo( SourceFolder );
			if( !SourceFolderInfo.Exists )
			{
				Program.Warning( "Source folder does not exist: " + SourceFolderInfo );
				return( false );
			}

			// Ensure the destination folder exists - create otherwise
			DirectoryInfo DestFolderInfo = new DirectoryInfo( DestFolder );
			if( !DestFolderInfo.Exists )
			{
				Directory.CreateDirectory( DestFolder );
			}

			foreach( FileInfo SourceInfo in SourceFolderInfo.GetFiles( FileSpec ) )
			{
				// Don't copy any file with the excluded extension
				if( ExcludeExtension != null && SourceInfo.Extension == ExcludeExtension )
				{
					continue;
				}

				string DestPathName = Path.Combine( DestFolderInfo.FullName, SourceInfo.Name );

				CopyFile(SourceInfo, DestPathName, false, DefaultRetryCount);
			}

			return( true );
		}

		/**
		 * Rename a file
		 */
		static public bool RenameFile( string Folder, string SourceName, string DestName )
		{
			Program.Log( " ... '" + Folder + "\\" + SourceName + "' -> '" + DestName + "'" );

			// Ensure the source file exists - fail otherwise
			string FullSourceName = Path.Combine( Folder, SourceName );
			FileInfo SourceInfo = new FileInfo( FullSourceName );
			if( !SourceInfo.Exists )
			{
				Program.Error( "File does not exist for renaming: " + FullSourceName );
				return ( false );
			}

			// Delete the destination file if it exists
			string FullDestName = Path.Combine( Folder, DestName );
			FileInfo DestInfo = new FileInfo( FullDestName );
			if( DestInfo.Exists && (FullSourceName != FullDestName))
			{
				DestInfo.IsReadOnly = false;
				DestInfo.Delete();
			}

			// Rename
			SourceInfo.MoveTo( DestInfo.FullName );

			return ( true );
		}

		/// <summary>
		/// This class exposes an interface that represents a limited subset of file system functionality, so the same
		/// code can work with a Zip file or a raw .app directory
		/// </summary>
		public abstract class FileSystemAdapter
		{
			public abstract byte[] ReadAllBytes(string RelativeFilename);
			public abstract Stream OpenRead(string RelativeFilename);
			public abstract void CloseFile(Stream InStream);
			public abstract void WriteAllBytes(string RelativeFilename, byte[] Data);
			public abstract void WriteStream(string RelativeFilename, Stream Data);
			public abstract void Close();

			public abstract IEnumerable<string> GetAllPayloadFiles();
		}

		/// <summary>
		/// Implementation of the FileSystemAdapter interface that handles the native file system
		/// </summary>
		public class RegularFileSystem : FileSystemAdapter
		{
			protected string MyAppDirectory;

			protected string GetNativePath(string RelativeFilename)
			{
				return Path.Combine(MyAppDirectory, RelativeFilename.Replace("/", "\\"));
			}

			protected string GetZipStyleRelativePath(string NativeFilename)
			{
				if (!NativeFilename.StartsWith(MyAppDirectory))
				{
					throw new InvalidDataException("Path is not rooted in app directory");
				}

				string RelativePath = NativeFilename.Substring(MyAppDirectory.Length);
				return RelativePath.Replace("\\", "/");
			}

			public RegularFileSystem(string AppDirectory)
			{
				MyAppDirectory = AppDirectory;
			}

			public override byte[] ReadAllBytes(string RelativeFilename)
			{
				return File.ReadAllBytes(GetNativePath(RelativeFilename));
			}

			public override Stream OpenRead(string RelativeFilename)
			{
				return File.OpenRead(GetNativePath(RelativeFilename));
			}

			public override void CloseFile(Stream InStream)
			{
			}

			public override void WriteAllBytes(string RelativeFilename, byte[] Data)
			{
				File.WriteAllBytes(GetNativePath(RelativeFilename), Data);
			}

			public override void WriteStream(string RelativeFilename, Stream Data)
			{
				Stream FileStream = File.OpenWrite(GetNativePath(RelativeFilename));
				Data.CopyTo(FileStream);
				FileStream.Close();
			}

			public override void Close()
			{
			}

			public override IEnumerable<string> GetAllPayloadFiles()
			{
				string[] SourceList = Directory.GetFiles(MyAppDirectory, "*.*", SearchOption.AllDirectories);

				List<string> Result = new List<string>();

				foreach (string Filename in SourceList)
				{
					Result.Add(GetZipStyleRelativePath(Filename));
				}

				return Result;
			}
		}

		/// <summary>
		/// Implementation of the FileSystemAdapter interface that handles Zip files
		/// </summary>
		public class ZipFileSystem : FileSystemAdapter
		{
			// Zip file
			protected ZipFile Zip;
			protected string InternalRootPath;

			// Pending writes, in case someone tries to read a file they've previously modified without saving the Zip.
			protected Dictionary<string, Stream> PendingWrites = new Dictionary<string, Stream>();

			/// Name of executable file name, so we can set correct attributes when zipping
			public String ExecutableFileName;


			/// <summary>
			/// Finds the root path in a given IPA (e.g., "Payload/UnrealGame.app/")
			/// </summary>
			public static string FindRootPathInIPA(ZipFile Zip)
			{
				// we want only the root Info.plist, not a plist inside an framework or extension
				string Info = ".app/Info.plist";

				var FileList = Zip.EntryFileNames;
				foreach (string TestFilename in FileList)
				{
					if (TestFilename.EndsWith(Info))
					{
						// the - 5 removes the .app portion and leaves the slash
						return TestFilename.Substring(0, TestFilename.Length - (Info.Length - 5));
					}
				}

				return null;
			}

			public ZipFileSystem(string Filename)
			{
				Zip = ZipFile.Read(Filename);
				InternalRootPath = FindRootPathInIPA(Zip);
			}

			public ZipFileSystem(ZipFile InZip)
			{
				this.Zip = InZip;
				InternalRootPath = FindRootPathInIPA(InZip);
			}

			public ZipFileSystem(ZipFile InZip, string RootPath)
			{
				this.Zip = InZip;
				InternalRootPath = RootPath;
			}

			protected string GetFileName(string RelativeFilename)
			{
				int SplitPoint = RelativeFilename.LastIndexOf("/") + 0;
				return RelativeFilename.Substring(SplitPoint);
			}

			protected static void SplitPath(string RelativeFilename, out string ZipPath, out string ZipFilename)
			{
				int SplitPoint = RelativeFilename.LastIndexOf("/") + 0;
				ZipPath = RelativeFilename.Substring(0, SplitPoint);
				ZipFilename = RelativeFilename.Substring(SplitPoint);
			}

			public override byte[] ReadAllBytes(string RelativeFilename)
			{
				Stream Source = OpenRead(RelativeFilename);
				byte[] Result = new byte[Source.Length];
				Source.Read(Result, 0, (int)Source.Length);
				Source.Close();
				return Result;
			}

			public override Stream OpenRead(string RelativeFilename)
			{
				string FullPath = InternalRootPath + RelativeFilename;

				Stream ReadStream;
				Stream PreviousStream;
				if (PendingWrites.TryGetValue(FullPath, out PreviousStream))
				{
					if (!PreviousStream.CanSeek)
					{
						// If data is not seekable, take a MemoryStream copy of the existing data
						// This could fail if the stream data is greater than 2 GB.
						MemoryStream MemStream = new MemoryStream();
						PreviousStream.CopyTo(MemStream);
						PreviousStream = MemStream;
						PendingWrites[FullPath] = PreviousStream;
					}

					// Return a proxy of the stored stream that we can read from without closing the underlying stream
					ReadStream = new ProxyReadStream(PreviousStream);
				}
				else
				{
					if (Zip.ContainsEntry(FullPath))
					{
						ZipEntry Entry = Zip[FullPath];
						if (Entry.UncompressedSize >= (2L * 1024 * 1024 * 1024))
						{
							Entry.Extract(Path.GetTempPath());
							ReadStream = File.OpenRead(Path.Combine(Path.GetTempPath(), Entry.FileName));
						}
						else
						{
							ReadStream = new MemoryStream((int)Entry.UncompressedSize);
							Zip[FullPath].Extract(ReadStream);
						}
					}
					else
					{
						string ErrorMessage = String.Format("Expected \"{0}\" in IPA but it was not found", FullPath);
						Program.Error(ErrorMessage);
						throw new Exception(ErrorMessage);
					}
				}

				ReadStream.Position = 0;
				return ReadStream;
			}

			public override void CloseFile(Stream InStream)
			{
				if ((InStream as System.IO.FileStream) != null)
				{
					InStream.Close();
					File.Delete((InStream as System.IO.FileStream).Name);
				}
			}

			public override void WriteAllBytes(string RelativeFilename, byte[] Data)
			{
				WriteStream(RelativeFilename, new MemoryStream(Data));
			}

			public override void WriteStream(string RelativeFilename, Stream Data)
			{
				string FullPath = InternalRootPath + RelativeFilename;

				// Add it to the pending writes list
				PendingWrites.Remove(FullPath);
				PendingWrites.Add(FullPath, Data);
			}

			/// <summary>
			/// Make sure every directory in a path has an explicit entry (the input must be a directory name only, with no filename)
			protected void EnsureDirectoriesExistInStrictPath(string DirectoryName)
			{
				int SlashIndex = DirectoryName.LastIndexOf("/");

				if (SlashIndex >= 0)
				{
					string SmallerPath = DirectoryName.Substring(0, SlashIndex);
					EnsureDirectoriesExistInStrictPath(SmallerPath);
				}

				DirectoryName = DirectoryName + "/";
				if (!Zip.ContainsEntry(DirectoryName))
				{
					Zip.AddDirectoryByName(DirectoryName);
				}
			}

			/// <summary>
			/// Make sure every directory in a fully qualified path has an explicit entry
			///  For a file named foo/bar.baz, ionic will only create one entry for foo/bar.baz,
			///  but the Mac or 7z would create a foo/ entry as well.
			/// </summary>
			protected void EnsureDirectoriesExist(string Filename)
			{
				int SlashIndex = Filename.LastIndexOf("/");

				if (SlashIndex >= 0)
				{
					string DirectoryName = Filename.Substring(0, SlashIndex);
					EnsureDirectoriesExistInStrictPath(DirectoryName);
				}
			}

			void CommitWrite(string FullPath, Stream StreamData)
			{
				// Create the directory if needed
				EnsureDirectoriesExist(FullPath);

				// Add the data
				Zip.UpdateEntry(FullPath, StreamData);
			}

			public override void Close()
			{
				// Flush out the pending writes
				foreach (var KVP in PendingWrites)
				{
					if (KVP.Value.CanSeek)
					{
						// Move back to the start of the file, in case we've used a StreamReadProxy to read
						KVP.Value.Seek(0, SeekOrigin.Begin);
					}
					CommitWrite(KVP.Key, KVP.Value);
				}
				PendingWrites.Clear();

				// Update permissions to be UNIX-style
				// Modify the file attributes of any added file to unix format
				foreach (ZipEntry E in Zip.Entries)
				{
					const byte FileAttributePlatform_NTFS = 0x0A;
					const byte FileAttributePlatform_UNIX = 0x03;
					const byte FileAttributePlatform_FAT = 0x00;

					const int UNIX_FILETYPE_NORMAL_FILE = 0x8000;
					//const int UNIX_FILETYPE_SOCKET = 0xC000;
					//const int UNIX_FILETYPE_SYMLINK = 0xA000;
					//const int UNIX_FILETYPE_BLOCKSPECIAL = 0x6000;
					const int UNIX_FILETYPE_DIRECTORY = 0x4000;
					//const int UNIX_FILETYPE_CHARSPECIAL = 0x2000;
					//const int UNIX_FILETYPE_FIFO = 0x1000;

					const int UNIX_EXEC = 1;
					const int UNIX_WRITE = 2;
					const int UNIX_READ = 4;


					int MyPermissions = UNIX_READ | UNIX_WRITE;
					int OtherPermissions = UNIX_READ;

					int PlatformEncodedBy = (E.VersionMadeBy >> 8) & 0xFF;
					int LowerBits = 0;

					// Try to preserve read-only if it was set
					bool bIsDirectory = E.IsDirectory;

					// Check to see if this 
					bool bIsExecutable = false;
					if (!String.IsNullOrEmpty(ExecutableFileName) && Path.GetFileNameWithoutExtension(E.FileName).Equals(ExecutableFileName, StringComparison.InvariantCultureIgnoreCase))
					{
						bIsExecutable = true;
					}

					if (bIsExecutable)
					{
						// The executable will be encrypted in the final distribution IPA and will compress very poorly, so keeping it
						// uncompressed gives a better indicator of IPA size for our distro builds
						E.CompressionLevel = CompressionLevel.None;
					}

					if ((PlatformEncodedBy == FileAttributePlatform_NTFS) || (PlatformEncodedBy == FileAttributePlatform_FAT))
					{
						FileAttributes OldAttributes = E.Attributes;
						//LowerBits = ((int)E.Attributes) & 0xFFFF;

						if ((OldAttributes & FileAttributes.Directory) != 0)
						{
							bIsDirectory = true;
						}

						// Permissions
						if ((OldAttributes & FileAttributes.ReadOnly) != 0)
						{
							MyPermissions &= ~UNIX_WRITE;
							OtherPermissions &= ~UNIX_WRITE;
						}
					}

					if (bIsDirectory || bIsExecutable)
					{
						MyPermissions |= UNIX_EXEC;
						OtherPermissions |= UNIX_EXEC;
					}

					// Re-jigger the external file attributes to UNIX style if they're not already that way
					if (PlatformEncodedBy != FileAttributePlatform_UNIX)
					{
						int NewAttributes = bIsDirectory ? UNIX_FILETYPE_DIRECTORY : UNIX_FILETYPE_NORMAL_FILE;

						NewAttributes |= (MyPermissions << 6);
						NewAttributes |= (OtherPermissions << 3);
						NewAttributes |= (OtherPermissions << 0);

						// Now modify the properties
						E.AdjustExternalFileAttributes(FileAttributePlatform_UNIX, (NewAttributes << 16) | LowerBits);
					}
				}

				Zip.Save();
			}
		

			public override IEnumerable<string> GetAllPayloadFiles()
			{
				List<string> Result = new List<string>();

				// Add files in the zip
				IEnumerable<string> SourceList = Zip.EntryFileNames;
				foreach (string Filename in SourceList)
				{
					ZipEntry Entry = Zip[Filename];

					if (!Entry.IsDirectory && Filename.StartsWith(InternalRootPath))
					{
						string RelativePath = Filename.Substring(InternalRootPath.Length);
						if (RelativePath.Length > 0)
						{
							Result.Add(RelativePath);
						}
					}
				}

				// Add files that are pending that weren't originally in the zip
				foreach (var KVP in PendingWrites)
				{
					string Filename = KVP.Key;
					if ((Zip[Filename] == null) && Filename.StartsWith(InternalRootPath))
					{
						string RelativePath = Filename.Substring(InternalRootPath.Length);
						Result.Add(RelativePath);
					}
				}

				return Result;
			}

			public void SetCompression(Ionic.Zlib.CompressionLevel Level)
			{
				Zip.CompressionLevel = Level;
			}
		}

		/// <summary>
		/// Implementation of the FileSystemAdapter interface that handles read-only access to Zip files
		/// </summary>
		public class ReadOnlyZipFileSystem : ZipFileSystem
		{
			public ReadOnlyZipFileSystem(ZipFile InZip, string RootPath)
				: base(InZip, RootPath)
			{
			}

			public ReadOnlyZipFileSystem(ZipFile InZip)
				: base(InZip)
			{
			}

			public ReadOnlyZipFileSystem(string Filename) : base(Filename)
			{
			}

			public override void Close()
			{
				Zip.Dispose();
				Zip = null;
			}

			public override void WriteAllBytes(string RelativeFilename, byte[] Data)
			{
				throw new InvalidOperationException("Cannot write files in a ReadOnlyZipFile object");
			}
		}
	}
}