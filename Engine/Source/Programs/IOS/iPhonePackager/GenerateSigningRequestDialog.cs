// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Drawing;
using System.Linq;
using System.Text;
using System.Windows.Forms;
using System.IO;
using iPhonePackager;

using Org.BouncyCastle.Pkcs;
using Org.BouncyCastle.Crypto.Parameters;
using Org.BouncyCastle.Crypto;
using System.Collections;
using Org.BouncyCastle.Asn1.X509;
using Org.BouncyCastle.Security;
using Org.BouncyCastle.Math;
using Org.BouncyCastle.Asn1;
using Org.BouncyCastle.OpenSsl;

namespace iPhonePackager
{
	public partial class GenerateSigningRequestDialog : Form
	{
		protected Dictionary<bool, Bitmap> CheckStateImages = new Dictionary<bool, Bitmap>();

		public GenerateSigningRequestDialog()
		{
			InitializeComponent();

			CheckStateImages.Add(false, iPhonePackager.Properties.Resources.GreyCheck);
			CheckStateImages.Add(true, iPhonePackager.Properties.Resources.GreenCheck);

			GenerateRequestButton.Enabled = false;
		}

		private void GenerateSigningRequestDialog_Load(object sender, EventArgs e)
		{
			SaveDialog.InitialDirectory = Environment.GetFolderPath(Environment.SpecialFolder.Desktop);
		}


		private void GenerateSigningRequestViaOpenSSL(string TargetCertRequestFileName, AsymmetricCipherKeyPair KeyPair)
		{
			// We expect openssl.exe to exist in the same directory as iPhonePackager
			string OpenSSLPath = Path.GetDirectoryName( Application.ExecutablePath ) + @"\openssl.exe";
			if (!File.Exists( OpenSSLPath ))
			{
				MessageBox.Show("A version of OpenSSL is required to generate certificate requests.  Please place OpenSSL.exe in Binaries\\DotNET\\IOS", Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
				return;
			}

			string EffectiveBuildPath = (Program.GameName.Length > 0) ? Config.BuildDirectory : Path.GetFullPath(".");

			// Create a temporary file to write the key pair out to (in a format that OpenSSL understands)
			string KeyFileName = Path.GetTempFileName();
			TextWriter KeyWriter = new StreamWriter(KeyFileName);

			PemWriter KeyWriterPEM = new PemWriter(KeyWriter);
			KeyWriterPEM.WriteObject(KeyPair);
			KeyWriter.Close();

			// Create a temporary file containing the configuration settings to drive OpenSSL
			string ConfigFileName = Path.GetTempFileName();
			TextWriter ConfigFile = new StreamWriter(ConfigFileName);
				
			ConfigFile.WriteLine("[ req ]");
			ConfigFile.WriteLine("distinguished_name     = req_distinguished_name");
			ConfigFile.WriteLine("prompt                 = no");

			ConfigFile.WriteLine("[ req_distinguished_name ]");
			ConfigFile.WriteLine("emailAddress           = {0}", EMailEditBox.Text);
			ConfigFile.WriteLine("commonName             = {0}", CommonNameEditBox.Text);
			ConfigFile.WriteLine("countryName            = {0}", System.Globalization.CultureInfo.CurrentCulture.TwoLetterISOLanguageName);

			ConfigFile.Close();

			// Invoke OpenSSL to generate the certificate request
			Program.Log("Running OpenSSL to generate certificate request...");

			string ResultsText;
			string Executable = OpenSSLPath;
			string Arguments = String.Format("req -new -nodes -out \"{0}\" -key \"{1}\" -config \"{2}\"",
				TargetCertRequestFileName, KeyFileName, ConfigFileName);
			Utilities.RunExecutableAndWait(Executable, Arguments, out ResultsText);

			Program.Log(ResultsText);

			if (!File.Exists(TargetCertRequestFileName))
			{
				Program.Error("... Failed to generate certificate request");
			}
			else
			{
				Program.Log("... Successfully generated certificate request '{0}'", TargetCertRequestFileName);
			}

			// Clean up the temporary files we created
			File.Delete(KeyFileName);
			File.Delete(ConfigFileName);
		}

		//@TODO: Generates a seemingly valid CSR but the Apple website doesn't accept this one (there is a context dependant 0 missing and a null missing compared to Keychain or OpenSSL generated ones, so the files aren't structurally identical)
		private void GenerateSigningRequestViaBouncyCastle(string TargetCertRequestFileName, AsymmetricCipherKeyPair KeyPair)
		{
			// Construct the request subject
			Hashtable Attributes = new Hashtable();
			Attributes.Add(X509Name.EmailAddress, EMailEditBox.Text);
			Attributes.Add(X509Name.CN, CommonNameEditBox.Text);
			Attributes.Add(X509Name.C, System.Globalization.RegionInfo.CurrentRegion.TwoLetterISORegionName);

			X509Name Subject = new X509Name(new ArrayList(Attributes.Keys), Attributes);

			// Generate a certificate signing request
			Pkcs10CertificationRequest CSR = new Pkcs10CertificationRequest(
				"SHA1withRSA",
				Subject,
				KeyPair.Public,
				null,
				KeyPair.Private);

			// Save the CSR to disk
			File.WriteAllBytes(TargetCertRequestFileName, CSR.GetEncoded());
		}

		private void GenerateRequestButton_Click(object sender, EventArgs e)
		{
			if (EMailEditBox.Text == "")
			{
				MessageBox.Show("An Email address is required to generate a signing request", Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
				return;
			}

			if (CommonNameEditBox.Text == "")
			{
				MessageBox.Show("A common name is required to generate a signing request", Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
				return;
			}

			AsymmetricCipherKeyPair KeyPair = null;
			try
			{
				KeyPair = CryptoAdapter.LoadKeyPairFromDiskBouncy(KeyPairFilenameBox.Text);
				if (KeyPair == null)
				{
					throw new InvalidDataException();
				}
			}
			catch (Exception)
			{
				MessageBox.Show("A public/private key pair is required to generate a signing request (failed to find or open specified key pair file)", Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
				return;
			}

			SaveDialog.FileName = "CertificateSigningRequest.csr";
			SaveDialog.DefaultExt = "csr";
			SaveDialog.Filter = ToolsHub.CertificateRequestFilter;
			SaveDialog.Title = "Generate Certificate Signing Request";
			if (SaveDialog.ShowDialog() == DialogResult.OK)
			{
				string EffectiveBuildPath = (Program.GameName.Length > 0) ? Config.BuildDirectory : Path.GetFullPath(".");
				string TargetCertRequestFileName = SaveDialog.FileName;

				GenerateSigningRequestViaOpenSSL(TargetCertRequestFileName, KeyPair);
				//GenerateSigningRequestViaBouncyCastle(TargetCertRequestFileName, KeyPair);

				// Close this dialog
				Close();
			}
		}

		private void GenerateSigningRequestDialog_FormClosing(object sender, FormClosingEventArgs e)
		{

		}

		private void GenerateSigningRequestDialog_FormClosed(object sender, FormClosedEventArgs e)
		{

		}

		private void GenerateKeyPairButton_Click(object sender, EventArgs e)
		{
			SaveDialog.FileName = "DeveloperKeyPair.key";
			SaveDialog.DefaultExt = "key";
			SaveDialog.Filter = ToolsHub.JustXmlKeysFilter;
			if (SaveDialog.ShowDialog() == DialogResult.OK)
			{
				string TargetKeyFilename = SaveDialog.FileName;

				// Generate a new 2048 bit RSA key and save it to disk
				System.Security.Cryptography.RSACryptoServiceProvider KeyPair = new System.Security.Cryptography.RSACryptoServiceProvider(2048);
				File.WriteAllText(TargetKeyFilename, KeyPair.ToXmlString(true));

				KeyPairFilenameBox.Text = TargetKeyFilename;
			}
		}

		private void ChooseKeyPairButton_Click(object sender, EventArgs e)
		{
			OpenDialog.Filter = ToolsHub.KeysFilter;
			if (OpenDialog.ShowDialog() == DialogResult.OK)
			{
				KeyPairFilenameBox.Text = OpenDialog.FileName;

				AsymmetricCipherKeyPair KeyPair = CryptoAdapter.LoadKeyPairFromDiskBouncy(KeyPairFilenameBox.Text);
				if (KeyPair == null)
				{
					MessageBox.Show("Unable to load key pair from disk, make sure it is in a supported format (xml or pkcs 12 key store)", Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
					KeyPairFilenameBox.Text = "";
				}
			}
		}

		private void ValidateCertSteps(object sender, EventArgs e)
		{
			bool bHasEmail = EMailEditBox.Text.Length > 0;
			bool bHasName = CommonNameEditBox.Text.Length > 0;
			bool bHasKeyPair = KeyPairFilenameBox.Text.Length > 0;

			EmailCheck.Image = CheckStateImages[bHasEmail];
			CommonNameCheck.Image = CheckStateImages[bHasName];
			KeyPairCheck.Image = CheckStateImages[bHasKeyPair];

			GenerateRequestButton.Enabled = bHasEmail && bHasName && bHasKeyPair;
		}
	}
}
