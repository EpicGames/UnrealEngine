/**
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Drawing;
using System.Linq;
using System.Text;
using System.Windows.Forms;
using Ionic.Zip;
using System.IO;
using System.Security.Cryptography.X509Certificates;

namespace iPhonePackager
{
	public partial class GraphicalResignTool : Form
	{

		public static GraphicalResignTool ActiveInstance = null;

		private string ExportImportPListFilename = "";
		private byte[] ImportedPListData = null;

		public static GraphicalResignTool GetActiveInstance()
		{
			if (ActiveInstance == null)
			{
				ActiveInstance = new GraphicalResignTool();
			}
			return ActiveInstance;
		}

		public GraphicalResignTool()
		{
			InitializeComponent();

			SwitchingInfoEditMode(null, null);
			SwitchingSigningMode(null, null);
			SwitchingMobileProvisionMode(null, null);

			InitializeGameSelectionBox();

			ActiveInstance = this;
		}

		/// <summary>
		/// Reads through directories that are peer to Binaries for any that match the *Game pattern
		/// and adds the games to a drop-down list that can be picked.
		/// </summary>
		void InitializeGameSelectionBox()
		{
		}

		private void GraphicalResignTool_Load(object sender, EventArgs e)
		{
			Text = Config.AppDisplayName + ": Signing tool";
		}


		private void InputIPABrowseButton_Click(object sender, EventArgs e)
		{
			string PickedFilename;
			if (ToolsHub.ShowOpenFileDialog(ToolsHub.IpaFilter, "Picking an application", "ipa", IPAFilenameEdit.Text, ref ToolsHub.ChoosingIpaDirectory, out PickedFilename))
			{
				IPAFilenameEdit.Text = PickedFilename;
				ImportedPListData = null;
			}
		}

		private void MobileProvisionBrowseButton_Click(object sender, EventArgs e)
		{
			string PickedFilename;
			if (ToolsHub.ShowOpenFileDialog(ToolsHub.MobileProvisionFilter, "Picking a mobile provision to embed", "mobileprovision", MobileProvisionEdit.Text, ref ToolsHub.ChoosingIpaDirectory, out PickedFilename))
			{
				MobileProvisionEdit.Text = PickedFilename;
			}
		}

		private void CertificateBrowseButton_Click(object sender, EventArgs e)
		{
			string PickedFilename;
			if (ToolsHub.ShowOpenFileDialog(ToolsHub.CertificatesFilter, "Picking a signing certificate to codesign with", "p12", CertificateEdit.Text, ref ToolsHub.ChoosingIpaDirectory, out PickedFilename))
			{
				CertificateEdit.Text = PickedFilename;
			}
		}

		void ShowError(string Context, Exception ex)
		{
			MessageBox.Show(String.Format("Error '{0}' encountered while {1}", ex.Message, Context), Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
		}

		void ShowErrorMessage(string Msg, params object [] Args)
		{
			MessageBox.Show(String.Format(Msg, Args), Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
		}

		private void ExportInfoButton_Click(object sender, EventArgs e)
		{
			try
			{
				FileOperations.FileSystemAdapter IPA = new FileOperations.ReadOnlyZipFileSystem(IPAFilenameEdit.Text);

				if (IPA != null)
				{
					saveFileDialog1.DefaultExt = "plist";
					saveFileDialog1.Title = "Exporting Info.plist";
					saveFileDialog1.Filter = ToolsHub.PListFilter;
					saveFileDialog1.FileName = ExportImportPListFilename;

					string CWD = Directory.GetCurrentDirectory();
					bool bDialogSucceeded = (saveFileDialog1.ShowDialog() == DialogResult.OK);
					Directory.SetCurrentDirectory(CWD);

					if (bDialogSucceeded)
					{
						ExportImportPListFilename = saveFileDialog1.FileName;
						byte[] DataToSave = IPA.ReadAllBytes("Info.plist");
						File.WriteAllBytes(ExportImportPListFilename, DataToSave);
					}
				}
			}
			catch (Exception ex)
			{
				ShowError("exporting Info.plist", ex);
			}
		}

		private void ImportInfoButton_Click(object sender, EventArgs e)
		{
			try
			{
				string PickedFilename;
				if (ToolsHub.ShowOpenFileDialog(ToolsHub.PListFilter, "Importing Info.plist", "plist", ExportImportPListFilename, ref ToolsHub.ChoosingIpaDirectory, out PickedFilename))
				{
					ExportImportPListFilename = PickedFilename;
					ImportedPListData = File.ReadAllBytes(ExportImportPListFilename);
				}
			}
			catch (Exception ex)
			{
				ShowError("importing Info.plist", ex);
			}
		}

		private void SwitchingInfoEditMode(object sender, EventArgs e)
		{
			ImportInfoButton.Enabled = RBReplaceInfoPList.Checked;
			ExportInfoButton.Enabled = RBReplaceInfoPList.Checked;
			DisplayNameEdit.Enabled = RBModifyInfoPList.Checked;
			BundleIDEdit.Enabled = RBModifyInfoPList.Checked;

			if (!RBReplaceInfoPList.Checked)
			{
				ImportedPListData = null;
			}
		}

		private void SwitchingSigningMode(object sender, EventArgs e)
		{
			CertificateEdit.Enabled = RBUseExplicitCert.Checked;
			CertificateBrowseButton.Enabled = RBUseExplicitCert.Checked;
		}

		private void SwitchingMobileProvisionMode(object sender, EventArgs e)
		{
			MobileProvisionEdit.Enabled = RBSpecifyMobileProvision.Checked;
			MobileProvisionBrowseButton.Enabled = RBSpecifyMobileProvision.Checked;
		}

		private void ResignButton_Click(object sender, EventArgs e)
		{
			saveFileDialog1.DefaultExt = "ipa";
			saveFileDialog1.FileName = "";
			saveFileDialog1.Filter = ToolsHub.IpaFilter;
			saveFileDialog1.Title = "Choose a filename for the re-signed IPA";

			string CWD = Directory.GetCurrentDirectory();
			bool bDialogSucceeded = (saveFileDialog1.ShowDialog() == DialogResult.OK);
			Directory.SetCurrentDirectory(CWD);

			if (bDialogSucceeded)
			{
				bool bSavedVerbose = Config.bVerbose;
				Config.bVerbose = true;
				Program.ProgressDialog.OnBeginBackgroundWork = delegate
				{
					string SrcFilename = IPAFilenameEdit.Text;
					string DestFilename = saveFileDialog1.FileName;

					// Delete the target location and copy the source there
					FileOperations.DeleteFile(DestFilename);
					FileOperations.CopyRequiredFile(SrcFilename, DestFilename);

					// Open the file
					FileOperations.ZipFileSystem FileSystem = new FileOperations.ZipFileSystem(DestFilename);
					FileSystem.SetCompression(CBCompressModifiedFiles.Checked ? Ionic.Zlib.CompressionLevel.BestCompression : Ionic.Zlib.CompressionLevel.None);

					// Resign and save the file
					ResignIPA(FileSystem);
					FileSystem.Close();
				};
				Config.bVerbose = bSavedVerbose;

				Program.ProgressDialog.ShowDialog();
			}
		}

		/// <summary>
		/// This class is used to override specific parts of the default resigning behavior
		/// </summary>
		class CustomCodeSigner : CodeSignatureBuilder
		{
			/// <summary>
			/// If non-null, this will be used instead of the existing embedded mobile provision
			/// </summary>
			public byte[] CustomMobileProvision = null;

			/// <summary>
			/// If non-null, this certificate will be used instead of doing a cert scan
			/// </summary>
			public X509Certificate2 CustomSigningCert = null;

			/// <summary>
			/// If non-null, this data will completely replace the original Info.plist
			/// </summary>
			public byte[] CustomInfoPList = null;

			protected override byte[] GetMobileProvision(string CFBundleIdentifier)
			{
				return (CustomMobileProvision != null) ? CustomMobileProvision : FileSystem.ReadAllBytes("embedded.mobileprovision");
			}

			protected override X509Certificate2 LoadSigningCertificate()
			{
				return (CustomSigningCert != null) ? CustomSigningCert : base.LoadSigningCertificate();
			}

			protected override Utilities.PListHelper LoadInfoPList()
			{
				if (CustomInfoPList != null)
				{
					return new Utilities.PListHelper(Encoding.UTF8.GetString(CustomInfoPList));
				}
				else
				{
					return base.LoadInfoPList();
				}
			}
		}

		/** 
		 * Using the stub IPA previously compiled on the Mac, create a new IPA with assets
		 */
		void ResignIPA(FileOperations.FileSystemAdapter FileSystem)
		{
			try
			{
				DateTime StartTime = DateTime.Now;

				// Configure the custom code signer
				CustomCodeSigner SigningContext = new CustomCodeSigner();
				SigningContext.FileSystem = FileSystem;

				// Custom mobile provision?
				if (RBSpecifyMobileProvision.Checked)
				{
					SigningContext.CustomMobileProvision = File.ReadAllBytes(MobileProvisionEdit.Text);
				}

				// Custom cert?
				if (RBUseExplicitCert.Checked)
				{
					string CertificatePassword = "";
					try
					{
						SigningContext.CustomSigningCert = new X509Certificate2(CertificateEdit.Text, CertificatePassword, X509KeyStorageFlags.PersistKeySet | X509KeyStorageFlags.Exportable | X509KeyStorageFlags.MachineKeySet);
						//Cert = new X509Certificate2(CertificateFilename, CertificatePassword, X509KeyStorageFlags.PersistKeySet | X509KeyStorageFlags.Exportable | X509KeyStorageFlags.MachineKeySet);
					}
					catch (System.Security.Cryptography.CryptographicException)
					{
						// Try once with a password
						if (PasswordDialog.RequestPassword(out CertificatePassword))
						{
							SigningContext.CustomSigningCert = new X509Certificate2(CertificateEdit.Text, CertificatePassword, X509KeyStorageFlags.PersistKeySet | X509KeyStorageFlags.Exportable | X509KeyStorageFlags.MachineKeySet);
							//Cert = new X509Certificate2(CertificateFilename, CertificatePassword, X509KeyStorageFlags.PersistKeySet | X509KeyStorageFlags.Exportable | X509KeyStorageFlags.MachineKeySet);
						}
						else
						{
							// User cancelled dialog, rethrow
							throw;
						}
					}
				}

				// Totally replace Info.plist? (just editing it is handled later)
				if (RBReplaceInfoPList.Checked)
				{
					SigningContext.CustomInfoPList = ImportedPListData;
				}

				// Start the resign process
				SigningContext.PrepareForSigning();

				// Partially modify Info.plist?
				if (RBModifyInfoPList.Checked)
				{
					SigningContext.Info.SetString("CFBundleDisplayName", DisplayNameEdit.Text);
					SigningContext.Info.SetString("CFBundleIdentifier", BundleIDEdit.Text);
				}

				// Re-sign the executable
				SigningContext.PerformSigning();

				// Save the IPA
				Program.Log("Saving IPA ...");
				FileSystem.Close();

				TimeSpan ElapsedTime = DateTime.Now - StartTime;
				Program.Log(String.Format("Finished re-signing IPA in took {0:0.00} s", ElapsedTime.TotalSeconds));

				MessageBox.Show("Re-signing succeeded!", Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Information);
			}
			catch (Exception ex)
			{
				ShowError("re-signing IPA", ex);
			}
		}

		private void CloseThisWindowButton_Click(object sender, EventArgs e)
		{
			Close();
		}

		private void InstallIPAButton_Click(object sender, EventArgs e)
		{
			string PickedFilename;
			if (ToolsHub.ShowOpenFileDialog(ToolsHub.IpaFilter, "Choose an IPA to install", "ipa", "", ref ToolsHub.ChoosingIpaDirectory, out PickedFilename))
			{
				Program.ProgressDialog.OnBeginBackgroundWork = delegate
				{
					DeploymentHelper.InstallIPAOnConnectedDevices(PickedFilename);
				};
				Program.ProgressDialog.ShowDialog();
			}
		}

		private void UninstallIPAButton_Click(object sender, EventArgs e)
		{
			
		}

		void ScanConnectedDevices()
		{
			ConnectedDeviceInformation [] DeviceMap = null;

			Program.ProgressDialog.OnBeginBackgroundWork = delegate
			{
				DeviceMap = DeploymentHelper.GetAllConnectedDevices();
			};

			if ((Program.ProgressDialog.ShowDialog() == DialogResult.OK) && (DeviceMap != null))
			{
				ConnectedDevicesList.BeginUpdate();
				ConnectedDevicesList.Items.Clear();

				foreach (var DeviceInfo in DeviceMap)
				{
					string DisplayName = DeviceInfo.DeviceName;
					string UDID = DeviceInfo.UDID;
					string DeviceType = DeviceInfo.DeviceType;

					ListViewItem NewItem = ConnectedDevicesList.Items.Add(DisplayName);
					NewItem.SubItems.Add(UDID);
					NewItem.SubItems.Add(DeviceType);
				}

				ConnectedDevicesList.EndUpdate();
			}
			else
			{
				Program.Error("Enumerating devices failed");
			}
		}

		private void FetchUDIDButton_Click(object sender, EventArgs e)
		{
			ScanConnectedDevices();
		}

		private void UDIDContextMenu_Opening(object sender, CancelEventArgs e)
		{

		}

		private void textBox1_TextChanged(object sender, EventArgs e)
		{

		}

		private void ImportCertificateButton3_Click(object sender, EventArgs e)
		{
			ToolsHub.TryInstallingCertificate_PromptForKey(null);
			RefreshProvisionsList();
		}

		private void ImportProvisionButton3_Click(object sender, EventArgs e)
		{
			ToolsHub.TryInstallingMobileProvision(null);
			RefreshProvisionsList();
		}

		void RefreshProvisionsList()
		{
		}

		private void RefreshProvisionListButton_Click(object sender, EventArgs e)
		{
			RefreshProvisionsList();
		}

		private void GraphicalResignTool_FormClosed(object sender, FormClosedEventArgs e)
		{
			if (ActiveInstance == this)
			{
				ActiveInstance = null;
			}
		}

		private void TabBook_SelectedIndexChanged(object sender, EventArgs e)
		{
			if (TabBook.SelectedTab == DeploymentPage)
			{
				ScanConnectedDevices();
			}
		}

		private void GameSelectionBox_SelectedIndexChanged(object sender, EventArgs e)
		{
			try
			{
				// THIS IS NO LONGER VALID, WE NEED A PATH TO THE GAME, AND RE-RUN CONFIG
				System.Windows.Forms.MessageBox.Show("This function is no longer supported - it will need work!");
//				 Program.GameName = GameSelectionBox.Items[GameSelectionBox.SelectedIndex] as string;
//				 Config.Initialize();
//				 RefreshProvisionsList();
			}
			catch
			{
			}
		}

		private void copyToolStripMenuItem_Click(object sender, EventArgs e)
		{
		}

		private void copyFullPathToProvisionToolStripMenuItem_Click(object sender, EventArgs e)
		{
		}

		private void GraphicalResignTool_Shown(object sender, EventArgs e)
		{
			TabBook_SelectedIndexChanged(sender, e);
		}

		private void copyUDIDToClipboardToolStripMenuItem_Click(object sender, EventArgs e)
		{
			string ClipboardText = "";
			foreach (ListViewItem i in ConnectedDevicesList.SelectedItems)
			{
				string Line = i.Text + " " + i.SubItems[1].Text;
				if (ClipboardText == "")
				{
					ClipboardText = Line;
				}
				else
				{
					ClipboardText = ClipboardText + "\n" + Line;
				}
			}

			Clipboard.SetText(ClipboardText);
		}

		static void CheckProvisionForUDID(MobileProvision Provision, string UDID, string Context)
		{
			if (Provision == null)
			{
				MessageBox.Show(String.Format("Failed to load or parse {0}", Context), Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
			}
			else
			{
				if (Provision.ContainsUDID(UDID))
				{
					MessageBox.Show(String.Format("Found the UDID {0} in {1}", UDID, Context), Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Information);
				}
				else
				{
					MessageBox.Show(String.Format("Warning: Failed to find UDID {0} in {1}!", UDID, Context), Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Warning);
				}
			}
		}

		void PickProvisionAndCheckForUDID(string UDID)
		{
			if (UDID != null)
			{
				string PickedFilename;
				if (ToolsHub.ShowOpenFileDialog(ToolsHub.MobileProvisionFilter, "Picking a mobile provision to check", "mobileprovision", "", ref ToolsHub.ChoosingIpaDirectory, out PickedFilename))
				{
					MobileProvision Provision = MobileProvisionParser.ParseFile(PickedFilename);

					CheckProvisionForUDID(
						Provision,
						UDID,
						String.Format("mobile provision {0}", Path.GetFileNameWithoutExtension(PickedFilename)));
				}
			}
		}

		private void CheckThisUDIDAgainstAMobileprovMenuItem_Click(object sender, EventArgs e)
		{
			string UDID = null;
			foreach (ListViewItem i in ConnectedDevicesList.SelectedItems)
			{
				UDID = i.SubItems[1].Text;
				break;
			}

			PickProvisionAndCheckForUDID(UDID);
		}

		void PickIpaAndCheckForUDID(string UDID)
		{
			if (UDID != null)
			{
				string PickedFilename;
				if (ToolsHub.ShowOpenFileDialog(ToolsHub.IpaFilter, "Picking an application to check", "ipa", "", ref ToolsHub.ChoosingIpaDirectory, out PickedFilename))
				{
					MobileProvision Provision = MobileProvisionParser.ParseIPA(PickedFilename);

					CheckProvisionForUDID(
						Provision,
						UDID,
						String.Format("embedded mobile provision in {0}", Path.GetFileName(PickedFilename)));
				}
			}
		}

		private void CheckUDIDInIPAMenuItem_Click(object sender, EventArgs e)
		{
			string UDID = null;
			foreach (ListViewItem i in ConnectedDevicesList.SelectedItems)
			{
				UDID = i.SubItems[1].Text;
				break;
			}

			PickIpaAndCheckForUDID(UDID);
		}

		private void CheckIPAForUDIDButton_Click(object sender, EventArgs e)
		{
			string UDID = ScanningUDIDEdit.Text.Trim();

			if (UDID.Length <= 0)
			{
				ShowErrorMessage("The UDID value is not properly set");
			}
			else
			{
				PickIpaAndCheckForUDID(UDID);
			}
		}

		private void CheckProvisionForUDIDButton_Click(object sender, EventArgs e)
		{
			string UDID = ScanningUDIDEdit.Text.Trim();

			if (UDID.Length <= 0)
			{
				ShowErrorMessage("The UDID value is not properly set");
			}
			else
			{
				PickProvisionAndCheckForUDID(UDID);
			}
		}

		string PickIPAAndFetchBundleIdentifier()
		{
			string PickedFilename;
			if (ToolsHub.ShowOpenFileDialog(ToolsHub.IpaFilter, "Choose an IPA to provide the bundle identifier", "ipa", "", ref ToolsHub.ChoosingIpaDirectory, out PickedFilename))
			{
				FileOperations.FileSystemAdapter IPA = new FileOperations.ReadOnlyZipFileSystem(PickedFilename);

				if (IPA != null)
				{
					byte[] InfoData = IPA.ReadAllBytes("Info.plist");
					Utilities.PListHelper Info = new Utilities.PListHelper(Encoding.UTF8.GetString(InfoData));

					string BundleID;
					if (Info.GetString("CFBundleIdentifier", out BundleID))
					{
						return BundleID;
					}
				}
			}

			return null;
		}

		private void BackupDocumentsButton_Click(object sender, EventArgs e)
		{
			string BundleID = PickIPAAndFetchBundleIdentifier();

			if (BundleID != null)
			{
				Program.ProgressDialog.OnBeginBackgroundWork = delegate
				{
					DeploymentHelper.BackupDocumentsDirectory(BundleID, Config.GetRootBackedUpDocumentsDirectory());
				};
				Program.ProgressDialog.ShowDialog();
			}
		}

		private void UninstallIPAButton_Click_1(object sender, EventArgs e)
		{
			string BundleID = PickIPAAndFetchBundleIdentifier();

			if (BundleID != null)
			{
				Program.ProgressDialog.OnBeginBackgroundWork = delegate
				{
					DeploymentHelper.UninstallIPAOnDevices(BundleID);
				};
				Program.ProgressDialog.ShowDialog();
			}
		}

		private void exportCertificateToolStripMenuItem_Click(object sender, EventArgs e)
		{
		}
	}
}
