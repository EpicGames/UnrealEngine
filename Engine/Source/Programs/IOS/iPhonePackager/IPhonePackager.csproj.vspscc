""
{
"FILE_VERSION" = "9237"
"ENLISTMENT_CHOICE" = "NEVER"
"PROJECT_FILE_RELATIVE_PATH" = ""
"NUMBER_OF_EXCLUDED_FILES" = "0"
"ORIGINAL_PROJECT_FILE_PATH" = ""
"NUMBER_OF_NESTED_PROJECTS" = "0"
"SOURCE_CONTROL_SETTINGS_PROVIDER" = "PROVIDER"
}
