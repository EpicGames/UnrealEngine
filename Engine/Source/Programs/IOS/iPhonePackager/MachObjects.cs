// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using iPhonePackager;
using System.Diagnostics;

using System.Security.Cryptography.X509Certificates;
using System.Security.Cryptography;
using System.Security.Cryptography.Pkcs;

using SysadminsLV.Asn1Parser;
using System.Xml;

/// <summary>
/// Minimal implementation of the Mach object file format
/// See: http://developer.apple.com/library/mac/#documentation/DeveloperTools/Conceptual/MachORuntime/Reference/reference.html for format details
/// And: http://www.opensource.apple.com/source/xnu/xnu-1456.1.26/EXTERNAL_HEADERS/mach-o/loader.h for constant values
/// 
/// Only supports 32 bit little-endian executables, and writing back out is not fully supported.
/// A few classes have support for patching individual fields.
/// </summary>
namespace MachObjectHandling
{
	// Values for CpuType
	public enum CpuType
	{
		ARCH_ABI64 = 0x1000000,
		I386 = 7,
		ARM = 12,
		POWERPC = 18,
		ARM64 = ARCH_ABI64 | ARM,
	}

	public class Bits
	{
		public enum Num
		{
			_32 = 32,
			_64 = 64,
		}
		public static int Bytes(Num num)
		{
			return (num == Num._64) ? sizeof(UInt64) : sizeof(UInt32);
		}
	}

	public abstract class RWContextBase
	{
		protected bool bIsOutput = false;
		public bool bStreamLittleEndian = true;
		protected Stack<long> Positions = new Stack<long>();
		protected byte[] SwapBuffer = new byte[8];

		// When a file is within another file, FileOffset can be set so all external operations can act as if it is just a single file.
		protected long ArchiveFileOffset;

		public void OpenFatArchiveAt(long AtOffset)
		{
			if (ArchiveFileOffset != 0)
			{
				throw new InvalidDataException(String.Format("Nested OpenFatAchriveAt() not supported:{0} {1} {2}", Position, ArchiveFileOffset, AtOffset));
			}

			// Start at beginning of file.
			PushPositionAndJump(0);
			ArchiveFileOffset = AtOffset;

			// Cause a seek to the ArchiveFileOffset.
			Position = 0;
		}
		public void CloseFatArchive()
		{
			if (ArchiveFileOffset == 0)
			{
				throw new InvalidDataException(String.Format("CloseFatArchive() called without Open {0} {1}.", Position, ArchiveFileOffset));
			}
			ArchiveFileOffset = 0;
			PopPosition();
		}

		public void PushPositionAndJump(long NewOffset)
		{
			Positions.Push(Position);
			Position = NewOffset;
		}

		public void PushPosition()
		{
			Positions.Push(Position);
		}

		public void PopPosition()
		{
			Position = Positions.Pop();
		}

		public abstract long Position
		{
			get;
			set;
		}

		public void VerifyStreamPosition(long StartingStreamPosition, long JustReadCount)
		{
			long ExpectedPosition = StartingStreamPosition + JustReadCount;
			if (Position != ExpectedPosition)
			{
				throw new InvalidDataException(String.Format("Stream offset is not as expected, {0} too {1} data!",
					bIsOutput ? "wrote" : "read",
					(Position < ExpectedPosition) ? "little" : "much"));
			}
		}

		public void VerifyStreamPosition(ref long StreamPosition, long JustReadCount)
		{
			StreamPosition += JustReadCount;
			VerifyStreamPosition(StreamPosition, 0);
		}
	}

	public class ReadingContext : RWContextBase
	{
		protected BinaryReader SR;

		public ReadingContext(BinaryReader Stream)
		{
			SR = Stream;
		}

		public override long Position
		{
			get { return SR.BaseStream.Position - ArchiveFileOffset; }
			set { SR.BaseStream.Position = value + ArchiveFileOffset; }
		}

		public UInt64 ReadUInt(Bits.Num Num)
		{
			if (Num == Bits.Num._64)
				return ReadUInt64();
			else
				return ReadUInt32();
		}

		public UInt32 ReadUInt32()
		{
			if (bStreamLittleEndian == BitConverter.IsLittleEndian)
			{
				return SR.ReadUInt32();
			}
			else
			{
				SR.Read(SwapBuffer, 0, 4);
				Array.Reverse(SwapBuffer, 0, 4);
				return BitConverter.ToUInt32(SwapBuffer, 0);
			}
		}

		public UInt64 ReadUInt64()
		{
			if (bStreamLittleEndian == BitConverter.IsLittleEndian)
			{
				return SR.ReadUInt64();
			}
			else
			{
				SR.Read(SwapBuffer, 0, 8);
				Array.Reverse(SwapBuffer, 0, 8);
				return BitConverter.ToUInt64(SwapBuffer, 0);
			}
		}

		public byte[] ReadBytes(int DataSize)
		{
			return SR.ReadBytes(DataSize);
		}

		public byte[] ReadBytes(UInt32 DataSize)
		{
			return SR.ReadBytes((int)DataSize);
		}

		public byte ReadByte()
		{
			return SR.ReadByte();
		}

		public string ReadFixedASCII(int Length)
		{
			return Utilities.ReadFixedASCII(SR, Length);
		}

		public string ReadASCIIZ()
		{
			// Find the end of the string
			long SavedPosition = Position;
			while (ReadByte() != 0)
			{
			}
			int Length = (int)(Position - SavedPosition);
			Position = SavedPosition;

			// Read it
			return ReadFixedASCII(Length);
		}
	}


	public class WritingPhase
	{
		public delegate void DelayedWriteCallback(WritingContext Context);

		public Queue<DelayedWriteCallback> PendingWrites = new Queue<DelayedWriteCallback>();

		public void Drain(WritingContext Context)
		{
			if (Config.bCodeSignVerbose)
			{
				Console.WriteLine("Draining phase with {0} writes", PendingWrites.Count);
			}

			while (PendingWrites.Count > 0)
			{
				if (Config.bCodeSignVerbose)
				{
					Console.WriteLine("  One delayed write at position = 0x{0:X}", Context.Position);
				}

				DelayedWriteCallback Job = PendingWrites.Dequeue();
				Job.Invoke(Context);
			}

			if (Config.bCodeSignVerbose)
			{
				Console.WriteLine("Finished draining phase (curpos = 0x{0:X})", Context.Position);
			}
		}

		public bool bWorkRemaining
		{
			get { return PendingWrites.Count > 0; }
		}
	}

	public class DeferredFieldU32or64
	{
		// The start of the section we're counting length for
		protected long AnchorPoint;

		// The offset the length is stored in
		public long WritePoint;

		public Bits.Num AddressSize;

		public DeferredFieldU32or64(WritingContext Context, long Base, Bits.Num Address)
		{
			AnchorPoint = Base;
			WritePoint = Context.Position;

			AddressSize = Address;
			Context.Position += Bits.Bytes(AddressSize);
		}

		public void Commit(WritingContext Context)
		{
			long CurrentPos = Context.Position;
			long Length = CurrentPos - AnchorPoint;

			if (AddressSize == Bits.Num._32)
			{
				Debug.Assert((Length >= UInt32.MinValue) && (Length <= UInt32.MaxValue));
			}

			Context.PushPositionAndJump(WritePoint);
			Context.WriteUInt((UInt64)Length, AddressSize);
			Context.PopPosition();
		}
	}

	public class LengthFieldU32or64 : DeferredFieldU32or64
	{
		public LengthFieldU32or64(WritingContext Context, long BaseOffset, Bits.Num Address)
			: base(Context, BaseOffset, Address)
		{
		}

		public void Rebase(long NewBaseOffset)
		{
			AnchorPoint = NewBaseOffset;
		}
	}

	public class OffsetFieldU32or64 : DeferredFieldU32or64
	{
		public OffsetFieldU32or64(WritingContext Context, long BaseOffset, Bits.Num Address)
			: base(Context, BaseOffset, Address)
		{
		}
	}

	public class WritingContext : RWContextBase
	{
		public WritingPhase CurrentPhase = new WritingPhase();

		Stack<WritingPhase> PendingPhases = new Stack<WritingPhase>();
		
		public void CreateNewPhase()
		{
			PendingPhases.Push(CurrentPhase);
			CurrentPhase = new WritingPhase();
		}

		public void Flush()
		{
			SW.Flush();
			SW.BaseStream.Flush();
		}

		/// <summary>
		/// Processes a phase worth of work, returning false when no work remains
		/// </summary>
		/// <returns>Returns true if more work remains</returns>
		public bool ProcessEntirePhase()
		{
			CurrentPhase.Drain(this);
			Debug.Assert(CurrentPhase.PendingWrites.Count == 0);

			if (PendingPhases.Count > 0)
			{
				CurrentPhase = PendingPhases.Pop();
			}

			return CurrentPhase.bWorkRemaining || (PendingPhases.Count > 0);
		}

		public LengthFieldU32or64 WriteDeferredLength(long AlreadyCountedBytes, Bits.Num Num)
		{
			return new LengthFieldU32or64(this, Position - AlreadyCountedBytes, Num);
		}

		public void CommitDeferredField(DeferredFieldU32or64 Field)
		{
			Field.Commit(this);
		}

		/// <summary>
		/// Emits an offset field that needs to be committed once the true offset is known
		/// </summary>
		/// <param name="AlreadyCountedBytes">The number of bytes ago that the offset is relative to.  Pass in 0 to be relative to the current output position.</param>
		/// <param name="Num">Number of bits this write will be.</param>
		/// <returns></returns>
		public OffsetFieldU32or64 WriteDeferredOffset(long AlreadyCountedBytes, Bits.Num Num)
		{
			return new OffsetFieldU32or64(this, Position - AlreadyCountedBytes, Num);
		}


		/// <summary>
		/// Emits an offset field that needs to be committed once the true offset is known
		/// </summary>
		/// <param name="AlreadyCountedBytes">The number of bytes ago that the offset is relative to.  Pass in 0 to be relative to the current output position.</param>
		/// <param name="Num">Number of bits this write will be.</param>
		/// <returns></returns>
		public OffsetFieldU32or64 WriteDeferredOffsetFrom(long BasePosition, Bits.Num Num)
		{
			return new OffsetFieldU32or64(this, BasePosition, Num);
		}

		protected BinaryWriter SW;

		public WritingContext(BinaryWriter Stream)
		{
			bIsOutput = true;
			SW = Stream;
		}

		public override long Position
		{
			get { return SW.BaseStream.Position - ArchiveFileOffset; }
			set { SW.BaseStream.Position = value + ArchiveFileOffset; }
		}

		public void CompleteWritingAndClose()
		{
			while (ProcessEntirePhase())
			{
			}
			SW.Close();
		}

		public void WriteZeros(long NumZeros)
		{
			if (NumZeros > 0)
			{
				byte[] ZeroList = new byte[NumZeros];

				//@TODO: Should be unnecessary
				for (int i = 0; i < ZeroList.Length; ++i)
				{
					ZeroList[i] = 0;
				}

				Write(ZeroList);
			}
		}

		public void WriteUInt(UInt64 Value, Bits.Num Num)
		{
			if (Num == Bits.Num._64)
				Write(Value);
			else
				Write((UInt32)Value);
		}

		public void Write(UInt32 Value)
		{
			if (bStreamLittleEndian == BitConverter.IsLittleEndian)
			{
				SW.Write(Value);
			}
			else
			{
				byte[] Buffer = BitConverter.GetBytes(Value);
				Array.Reverse(Buffer, 0, 4);
				SW.Write(Buffer, 0, 4);
			}
		}

		public void Write(UInt64 Value)
		{
			if (bStreamLittleEndian == BitConverter.IsLittleEndian)
			{
				SW.Write(Value);
			}
			else
			{
				byte[] Buffer = BitConverter.GetBytes(Value);
				Array.Reverse(Buffer, 0, 8);
				SW.Write(Buffer, 0, 8);
			}
		}

		public void Write(int Value)
		{
			Write((UInt32)Value);
		}

		public void Write(byte[] Buffer)
		{
			SW.Write(Buffer);
		}

		public void Write(byte[] Buffer, int Offset, int Length)
		{
			SW.Write(Buffer, Offset, Length);
		}

		public void Write(byte Value)
		{
			SW.Write(Value);
		}

		public void WriteFixedASCII(string Value, int BufferLength)
		{
			Utilities.WriteFixedASCII(SW, Value, BufferLength);
		}

		public void WriteAbsoluteOffsetAndDelayedData(byte[] Data, long Alignment, Bits.Num Num)
		{
			WriteOffsetAndDelayedData(Data, Alignment, 0, Num);
		}

		public void WriteOffsetAndDelayedData(byte[] Data, long Alignment, long OffsetFrom, Bits.Num Num)
		{
			if ((Data == null) || (Data.Length == 0))
			{
				SW.Write((UInt32)0);
			}
			else
			{
				OffsetFieldU32or64 Offset = WriteDeferredOffsetFrom(OffsetFrom, Num);
				CurrentPhase.PendingWrites.Enqueue(delegate(WritingContext Context)
				{
					long ModPosition = Position % Alignment;
					if (ModPosition != 0)
					{
						WriteZeros(Alignment - ModPosition);
					}

					CommitDeferredField(Offset);
					Write(Data);
				});
			}
		}
	}


	/// <summary>
	/// Information about a single section within a segment
	/// Warning: Contains absolute file offsets
	/// </summary>
	class MachSection
	{
		string SectionName;
		string SegmentName;
		UInt64 Addr;
		UInt64 Size;
		//UInt32 Offset;
		UInt32 LogAlignment;
		UInt32 RelocationOffset;
		UInt32 NumRelocations;
		int Flags;
		UInt32 Reserved1;
		UInt32 Reserved2;
		// Only in 64 bit - Docs do not say so, but header file does http://llvm.org/docs/doxygen/html/Support_2MachO_8h_source.html.
		UInt32 Reserved3; 

		Bits.Num AddressSize;

		byte[] SectionData;

		// These section types have an offset of 0 and no bytes of file data
		const int S_ZEROFILL = 0x1;
		const int S_GB_ZEROFILL = 0xC;

		public MachSection(Bits.Num Num)
		{
			AddressSize = Num;
		}

		public int SectionType
		{
			get { return ((int)Flags) & 0xFF; }
			set
			{
				Flags = (Flags & ~0xFF) | (value & 0xFF);
			}
		}

		public void Read(ReadingContext SR)
		{
			SectionName = SR.ReadFixedASCII(16);
			SegmentName = SR.ReadFixedASCII(16);
			Addr = SR.ReadUInt(AddressSize);
			Size = SR.ReadUInt(AddressSize);
			UInt32 FileOffset = SR.ReadUInt32();
			LogAlignment = SR.ReadUInt32();
			RelocationOffset = SR.ReadUInt32();
			NumRelocations = SR.ReadUInt32();
			Flags = (int)SR.ReadUInt32();
			Reserved1 = SR.ReadUInt32();
			Reserved2 = SR.ReadUInt32();

			if (AddressSize == Bits.Num._64)
				Reserved3 = SR.ReadUInt32();

			if ((SectionType == S_ZEROFILL) || (SectionType == S_GB_ZEROFILL))
			{
				Debug.Assert(FileOffset == 0);
				SectionData = null;
			}
			else
			{
				SR.PushPositionAndJump((long)FileOffset);
				SectionData = SR.ReadBytes((int)Size);	  
				SR.PopPosition();
			}

			if (Config.bCodeSignVerbose)
			{
				Console.WriteLine("  v Read Section '{0}' in segment '{1}' with size {2} and offset 0x{3:X} and align {4} and flags {5:X}", SectionName, SegmentName, Size, FileOffset, 1 << (byte)LogAlignment, Flags);
			}
		}

		public void Write(WritingContext SW)
		{
			SW.WriteFixedASCII(SectionName, 16);
			SW.WriteFixedASCII(SegmentName, 16);
			SW.WriteUInt(Addr, AddressSize);
			SW.WriteUInt(Size, AddressSize);
			SW.WriteAbsoluteOffsetAndDelayedData(SectionData, 1 << (byte)LogAlignment, AddressSize);
			SW.Write(LogAlignment);
			SW.Write(RelocationOffset);
			SW.Write(NumRelocations);
			SW.Write((UInt32)Flags);
			SW.Write(Reserved1);
			SW.Write(Reserved2);

			if (AddressSize == Bits.Num._64)
				SW.Write(Reserved3);
		}
	}

	public abstract class MachLoadCommand
	{
		public UInt32 Command;
		public bool RequiredForDynamicLoad;

		/// <summary>
		/// The offset (in bytes) from the start of the file when loading this command from a Mach
		/// Object file, or -1 if this was created in memory.  Only used for special modification code
		/// that modifies individual commands without fully understanding the file format, since MachO
		/// files aren't designed in such a way that allows piecemeal modifications...
		/// </summary>
		public long StartingLoadOffset = -1;

		/// <summary>
		/// Information on a segment (payload type: MachLoadCommandSegment)
		/// </summary>
		public const UInt32 LC_SEGMENT = 0x01;
		public const UInt32 LC_SEGMENT_64 = 0x19;

		/// <summary>
		/// Symbol and string tables (payload type: MachLoadCommandSymbolTable)
		/// </summary>
		public const UInt32 LC_SYMTAB = 0x02;

		/// <summary>
		/// Defines information about the main thread (payload type: MachLoadCommandMainThreadInfo)
		/// </summary>
		public const UInt32 LC_THREAD = 0x04;
		public const UInt32 LC_UNIXTHREAD = 0x05;

		/// <summary>
		/// Information about the parts of the symbol table that are used for dynamic linking (payload type: MachLoadCommandDynamicSymbolTable)
		/// </summary>
		public const UInt32 LC_DYSYMTAB = 0x0B;

		// Specifies a dynamically linked library (payload type: MachLoadCommandDylib)
		public const UInt32 LC_LOAD_DYLIB = 0x0C;
		public const UInt32 LC_ID_DYLIB = 0x0D;

		/// <summary>
		/// Specifies the name of the dynamic linker to use (payload type: MachLoadCommandDynamicLinkerName)
		/// </summary>
		public const UInt32 LC_LOAD_DYLINKER = 0x0E;
		public const UInt32 LC_ID_DYLINKER = 0x0F;

		/// <summary>
		/// Specifies a weak dynamically linked library (payload type: MachLoadCommandReferencedDLL)
		/// </summary>
		public const UInt32 LC_LOAD_WEAK_DYLIB = 0x18;

		/// <summary>
		/// Specifies a GUID (payload type: MachLoadCommandUUID)
		/// </summary>
		public const UInt32 LC_UUID = 0x1B;

		/// <summary>
		/// Specifies the code signature information
		/// </summary>
		public const UInt32 LC_CODE_SIGNATURE = 0x1D;
		
		/// <summary>
		/// Specifies an encrypted segment (payload type: MachLoadCommandEncryptedSegmentInfo)
		/// </summary>
		public const UInt32 LC_ENCRYPTION_INFO = 0x21;

		/// <summary>
		/// Specifies information the dynamic loader needs to load this object (payload type: MachLoadCommandDynamicLoaderInfo)
		/// </summary>
		public const UInt32 LC_DYLD_INFO = 0x22;


		protected abstract void PackageData(WritingContext SW);
		protected abstract void UnpackageData(ReadingContext SR, int CommandSize);

		public static MachLoadCommand CreateFromStream(ReadingContext SR)
		{
			long StartingStreamPosition = SR.Position;

			// Read the code and size (common to all commands)
			UInt32 CommandCode = SR.ReadUInt32();
			UInt32 CommandSize = SR.ReadUInt32();

			UInt32 EffectiveCommand = CommandCode & ~(1 << 31);
			MachLoadCommand Result;
			switch (EffectiveCommand)
			{
				case LC_SEGMENT:
					Result = new MachLoadCommandSegment(Bits.Num._32);
					break;

				case LC_SEGMENT_64:
					Result = new MachLoadCommandSegment(Bits.Num._64);
					break;

				case LC_SYMTAB:
					Result = new MachLoadCommandSymbolTable();
					break;

				case LC_THREAD:
				case LC_UNIXTHREAD:
					Result = new MachLoadCommandMainThreadInfo();
					break;

				case LC_DYSYMTAB:
					Result = new MachLoadCommandDynamicSymbolTable();
					break;

				case LC_LOAD_DYLIB:
				case LC_LOAD_WEAK_DYLIB:
				case LC_ID_DYLIB:
					Result = new MachLoadCommandDylib();
					break;

				case LC_LOAD_DYLINKER:
				case LC_ID_DYLINKER:
					Result = new MachLoadCommandDynamicLinkerName();
					break;

				case LC_UUID:
					Result = new MachLoadCommandUUID();
					break;

				case LC_CODE_SIGNATURE:
					Result = new MachLoadCommandCodeSignature();
					break;

				case LC_ENCRYPTION_INFO:
					Result = new MachLoadCommandEncryptedSegmentInfo();
					break;

				case LC_DYLD_INFO:
					Result = new MachLoadCommandDynamicLoaderInfo();
					break;

				default:
					Result = new MachLoadCommandOpaque();
					break;
			}

			// MSB of CommandCode is LC_REQ_DYLD or 0, indicating if the command can (theoretically) be ignored
			// or is critical to loading
			Result.StartingLoadOffset = StartingStreamPosition;
			Result.Command = EffectiveCommand;
			Result.RequiredForDynamicLoad = (CommandCode >> 31) != 0;

			Result.UnpackageData(SR, (int)CommandSize);

			SR.VerifyStreamPosition(StartingStreamPosition, CommandSize);
			return Result;
		}

		public void Write(WritingContext SW)
		{
			long StartingPosition = SW.Position;

			// Write the command and length
			SW.Write(Command);
			LengthFieldU32or64 Length = SW.WriteDeferredLength(4, Bits.Num._32);

			// Write the data
			PackageData(SW);

			// Write any pad bytes and commit the length
			SW.WriteZeros((SW.Position - StartingPosition) % 4);
			SW.CommitDeferredField(Length);
		}
	}

	/// <summary>
	/// Opaque mach load command
	/// </summary>
	class MachLoadCommandOpaque : MachLoadCommand
	{
		protected byte[] MyCommandData;

		protected override void PackageData(WritingContext SW)
		{
			SW.Write(MyCommandData, 0, MyCommandData.Length);
		}

		protected override void UnpackageData(ReadingContext SR, int CommandSize)
		{
			int DataSize = (int)CommandSize - (2 * sizeof(UInt32));
			Debug.Assert(DataSize >= 0);
			MyCommandData = SR.ReadBytes(DataSize);
		}

		/// <summary>
		/// Converts the N bytes at starting at Offset of the payload buffer to text (standard hex editor rules)
		/// If N is 0, N is set to the size of the payload
		/// </summary>
		protected string PayloadToString(int N, int Offset)
		{
			if (N == 0)
			{
				N = int.MaxValue;
			}
			N = Math.Min(N, MyCommandData.Length - Offset);

			if (N <= 0)
			{
				return "";
			}

			char[] Buffer = new char[N];
			int WriteIndex = 0;
			while (WriteIndex < Buffer.Length)
			{
				byte Value = MyCommandData[WriteIndex + Offset];
				if (Value > 0)
				{
					Buffer[WriteIndex] = ((Value >= 0x20) && (Value < 0x7F)) ? (char)Value : '.';
					WriteIndex++;
				}
				else
				{
					break;
				}
			}

			return new String(Buffer, 0, WriteIndex);
		}

		public override string ToString()
		{
			return String.Format("Unknown 0x{0:X} with {1} bytes of data", Command, MyCommandData.Length);
		}
	}


	/// <summary>
	/// A blob of data in the __LINKEDIT segment
	/// Warning: Contains absolute file offsets
	/// </summary>
	class MachLoadCommandLinkEditBlob : MachLoadCommand
	{
		public UInt32 BlobFileOffset;
		public UInt32 BlobFileSize;

		protected override void PackageData(WritingContext SW)
		{
			SW.Write(BlobFileOffset);
			SW.Write(BlobFileSize);
		}

		protected override void UnpackageData(ReadingContext SR, int CommandSize)
		{
			BlobFileOffset = SR.ReadUInt32();
			BlobFileSize = SR.ReadUInt32();
		}

		public override string ToString()
		{
			return String.Format("Blob at offset 0x{0:X} with a size of {1} KB", BlobFileOffset, BlobFileSize / 1024.0f);
		}
	}

	/// <summary>
	/// Payload for LC_SEGMENT && LC_SEGMENT_64
	/// Warning: Contains absolute file offsets (what an awful file format!)
	/// </summary>
	class MachLoadCommandSegment : MachLoadCommand
	{
		public string SegmentName;

		public UInt64 VirtualAddress;
		public UInt64 VirtualSize;

		public UInt64 FileOffset;
		public UInt64 FileSize;

		public UInt32 MaxProt;
		public UInt32 InitProt;
		public List<MachSection> Sections = new List<MachSection>();
		public UInt32 Flags;

		public Bits.Num AddressSize;

		public MachLoadCommandSegment(Bits.Num Addressing)
		{
			AddressSize = Addressing;
		}

		/// <summary>
		/// Patches just the file length of this segment in an existing file
		/// </summary>
		public void PatchFileLength(WritingContext SW, UInt32 NewLength)
		{
			Debug.Assert(StartingLoadOffset >= 0);

			if (Config.bCodeSignVerbose)
			{
				Console.WriteLine("ZZZZZZZZZZZZZZZZ");
				Console.WriteLine(" Segment Length from 0x{0:X} to 0x{1:X} (delta {2})", FileSize, NewLength, (long)NewLength - (long)FileSize);
				Console.WriteLine("ZZZZZZZZZZZZZZZZ");
			}

			FileSize = NewLength;

			long PatchOffset = StartingLoadOffset + (2 * sizeof(UInt32)); // Command, CommandLength
			PatchOffset += 16; // SegmentName
			PatchOffset += 1 * Bits.Bytes(AddressSize); // VirtualAddress
			// Adjust VirtualSize to accomodate a larger FileSize
			if (FileSize > VirtualSize)
			{
				// VirtualSize is the page aligned memory size required to load FileSize
				ulong PageAlignmentMask = (AddressSize == Bits.Num._64 ? (ulong)0x4000 : (ulong)0x1000) - 1;
				VirtualSize = (FileSize + PageAlignmentMask) & ~PageAlignmentMask;
				SW.PushPositionAndJump(PatchOffset);
				SW.WriteUInt(VirtualSize, AddressSize);
				SW.PopPosition();
			}
			PatchOffset += 2 * Bits.Bytes(AddressSize); // VirtualSize, FileOffset

			SW.PushPositionAndJump(PatchOffset);
			SW.WriteUInt(FileSize, AddressSize);
			SW.PopPosition();
		}

		protected override void PackageData(WritingContext SW)
		{
			// Write the segment load command
			SW.WriteFixedASCII(SegmentName, 16);
			SW.WriteUInt(VirtualAddress, AddressSize);
			SW.WriteUInt(VirtualSize, AddressSize);

			// Offset to first segment
			//@TODO: These file offsets and file lengths aren't correct (compared to the original MachO's)
			OffsetFieldU32or64 FileOffset = SW.WriteDeferredOffsetFrom(0, AddressSize);
			LengthFieldU32or64 FileLength = SW.WriteDeferredLength(0, AddressSize);

			SW.Write(MaxProt);
			SW.Write(InitProt);
			SW.Write(Sections.Count);
			SW.Write(Flags);

			// Enqueue a job to commit the file offset to the first section
			SW.CurrentPhase.PendingWrites.Enqueue(delegate(WritingContext Context)
			{
				FileLength.Rebase(Context.Position);
				FileOffset.Commit(Context);
			});

			// Write the sections belonging to the segment
			foreach (MachSection Section in Sections)
			{
				Section.Write(SW);
			}

			// Enqueue a job to commit the length of data in all the sections
			SW.CurrentPhase.PendingWrites.Enqueue(delegate(WritingContext Context)
			{
				FileLength.Commit(Context);
			});
		}

		public override string ToString()
		{
			return String.Format("Segment '{0}' with {1} sections (Offset 0x{2:X} Size {3})", SegmentName, Sections.Count, FileOffset, FileSize);
		}

		protected override void UnpackageData(ReadingContext SR, int CommandSize)
		{
			// Read the segment
			SegmentName = SR.ReadFixedASCII(16);
			VirtualAddress = SR.ReadUInt(AddressSize);
			VirtualSize = SR.ReadUInt(AddressSize);
			FileOffset = SR.ReadUInt(AddressSize);
			FileSize = SR.ReadUInt(AddressSize);
			MaxProt = SR.ReadUInt32();
			InitProt = SR.ReadUInt32();
			UInt32 SectionCount = SR.ReadUInt32();
			Flags = SR.ReadUInt32();

			// Read the segment data
			//SR.PushPositionAndJump(FileOffset);
			//FileData = SR.ReadBytes(FileSize);
			//SR.PopPosition();

			// Read the sections belonging to the segment
			for (int SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
			{
				MachSection Section = new MachSection(AddressSize);
				Section.Read(SR);
				Sections.Add(Section);
			}
		}
	}

	/// <summary>
	/// Payload for LC_SYMTAB
	/// Warning: Contains absolute offsets
	/// </summary>
	class MachLoadCommandSymbolTable : MachLoadCommand
	{
		UInt32 SymbolTableOffset;
		UInt32 SymbolCount;
		UInt32 StringTableOffset;
		UInt32 StringTableSize;

		protected override void PackageData(WritingContext SW)
		{
			SW.Write(SymbolTableOffset);
			SW.Write(SymbolCount);
			SW.Write(StringTableOffset);
			SW.Write(StringTableSize);
		}

		protected override void UnpackageData(ReadingContext SR, int CommandSize)
		{
			SymbolTableOffset = SR.ReadUInt32();
			SymbolCount = SR.ReadUInt32();
			StringTableOffset = SR.ReadUInt32();
			StringTableSize = SR.ReadUInt32();
		}

		public override string ToString()
		{
			return String.Format("Symbol Table with {0} symbols at offset 0x{1:X} and a {2} KB string table at offset 0x{3:X}",
				SymbolCount, SymbolTableOffset,
				StringTableSize / 1024.0f, StringTableOffset);
		}
	}

	/// <summary>
	/// Payload for LC_THREAD and LC_UNIXTHREAD
	/// </summary>
	class MachLoadCommandMainThreadInfo : MachLoadCommandOpaque
	{
		public override string ToString()
		{
			switch (Command)
			{
				case LC_THREAD:
					return "Main thread configuration (no default stack)";
				case LC_UNIXTHREAD:
					return "Main thread configuration (with a stack)";
				default:
					return base.ToString();
			}
		}
	}

	/// <summary>
	/// Payload for LC_DYSYMTAB
	/// Warning: Contains absolute file offsets
	/// </summary>
	class MachLoadCommandDynamicSymbolTable : MachLoadCommand
	{
		UInt32 [] Payload = new UInt32[18];
		// 6,8,10,12,14,16 are offsets :(

		protected override void PackageData(WritingContext SW)
		{
			for (int i = 0; i < Payload.Length; ++i)
			{
				SW.Write(Payload[i]);
			}
		}

		protected override void UnpackageData(ReadingContext SR, int CommandSize)
		{
			for (int i = 0; i < Payload.Length; ++i)
			{
				Payload[i] = SR.ReadUInt32();
			}
		}

		public override string ToString()
		{
			return "Information on symbols needed for dynamic linking and loading";
		}
	}
	
	/// <summary>
	/// Payload for LC_LOAD_DYLIB, LC_LOAD_WEAK_DYLIB, and LC_ID_DYLIB
	/// </summary>
	class MachLoadCommandDylib : MachLoadCommandOpaque
	{
		public override string ToString()
		{
			string Result;
			switch (Command)
			{
				case LC_LOAD_DYLIB:
					Result = "Load Dynamic Library";
					break;
				case LC_LOAD_WEAK_DYLIB:
					Result = "Load Weak Dynamic Library";
					break;
				case LC_ID_DYLIB:
					Result = "Dynamic Library Install Name";
					break;
				default:
					Result = base.ToString();
					return Result;
			}

			return String.Format("{0} '{1}'", Result, PayloadToString(0, 16));
		}
	}
	
	/// <summary>
	/// Payload for LC_LOAD_DYLINKER and LC_ID_DYLINKER
	/// </summary>
	class MachLoadCommandDynamicLinkerName : MachLoadCommandOpaque
	{
		public override string ToString()
		{
			return String.Format("Dynamic linker name '{0}'", PayloadToString(0, 4));
		}
	}

	/// <summary>
	/// Payload for LC_UUID
	/// </summary>
	class MachLoadCommandUUID : MachLoadCommandOpaque
	{
		public override string ToString()
		{
			Guid ID = new Guid(MyCommandData);
			return String.Format("UUID: {0}", ID);
		}
	}

	/// <summary>
	/// Payload for LC_ENCRYPTION_INFO
	/// </summary>
	class MachLoadCommandEncryptedSegmentInfo : MachLoadCommand
	{
		UInt32 EncryptedFileOffset;
		UInt32 EncryptedFileSize;
		UInt32 EncryptionMode;

		protected override void PackageData(WritingContext SW)
		{
			SW.Write(EncryptedFileOffset);
			SW.Write(EncryptedFileSize);
			SW.Write(EncryptionMode);
		}

		protected override void UnpackageData(ReadingContext SR, int CommandSize)
		{
 			EncryptedFileOffset = SR.ReadUInt32();
 			EncryptedFileSize = SR.ReadUInt32();
 			EncryptionMode = SR.ReadUInt32();
		}

		public override string ToString()
		{
			return String.Format("Encrypted segment at offset 0x{0:X} of size {1} KB in mode {2}", EncryptedFileOffset, EncryptedFileSize / 1024.0f, EncryptionMode);
		}
	}

	/// <summary>
	/// Payload for LC_DYLD_INFO
	/// Warning: Contains absolute file offsets
	/// </summary>
	class MachLoadCommandDynamicLoaderInfo : MachLoadCommand
	{
		UInt32[] Payload = new UInt32[10];
		// 0,2,4,6,8 are offsets :(

		protected override void PackageData(WritingContext SW)
		{
			for (int i = 0; i < Payload.Length; ++i)
			{
				SW.Write(Payload[i]);
			}
		}


		protected override void UnpackageData(ReadingContext SR, int CommandSize)
		{
			for (int i = 0; i < Payload.Length; ++i)
			{
				Payload[i] = SR.ReadUInt32();
			}
		}

		public override string ToString()
		{
			return "Compressed information needed for dynamic loading";
		}
	}

	class SigningBlob
	{
		const UInt32 Magic = 0xC00CDEFA;
	}

	public class MachObjectFile
	{
		public UInt32 Magic;
		public UInt32 CpuType;
		public UInt32 CpuSubType;
		public UInt32 FileType;
		// NumCommands
		// SizeOfCommand
		public UInt32 Flags;
		public UInt32 Reserved64;   // Only in mach_header_64

		public List<MachLoadCommand> Commands = new List<MachLoadCommand>();

		// Values for Magic
		public const UInt32 MH_MAGIC = 0xFEEDFACE;
		public const UInt32 MH_CIGAM = 0xCEFAEDFE;
		public const UInt32 MH_MAGIC_64 = 0xFEEDFACF;
		public const UInt32 MH_CIGAM_64 = 0xCFFAEDFE;

		// Values for FileType
		public const UInt32 MH_EXECUTE = 2;

		public const int MachHeaderPad = 0x1A60;

		public void Read(ReadingContext SR)
		{
			long ExpectedStreamPosition = 0;
			int NumInt = 7;

			// Read the header
			Magic = SR.ReadUInt32();
			CpuType = SR.ReadUInt32();
			CpuSubType = SR.ReadUInt32();
			FileType = SR.ReadUInt32();
			UInt32 NumCommands = SR.ReadUInt32();
			UInt32 SizeOfCommands = SR.ReadUInt32();
			Flags = SR.ReadUInt32();

			if (Magic == MH_MAGIC_64)
			{
				Reserved64 = SR.ReadUInt32();
				NumInt++;
			}
			SR.VerifyStreamPosition(ref ExpectedStreamPosition, NumInt * sizeof(UInt32));

			// Read the commands
			Commands.Clear();
			for (int LoadCommandIndex = 0; LoadCommandIndex < NumCommands; ++LoadCommandIndex)
			{
				MachLoadCommand Command = MachLoadCommand.CreateFromStream(SR);
				Commands.Add(Command);

				if (Config.bCodeSignVerbose)
				{
					Console.WriteLine(" Read command: {0}", Command.ToString());
				}
			}
			SR.VerifyStreamPosition(ref ExpectedStreamPosition, SizeOfCommands);
		}

		public void Write(WritingContext Context)
		{
			// Write the header
			Context.Write(Magic);
			Context.Write(CpuType);
			Context.Write(CpuSubType);
			Context.Write(FileType);

			Context.Write((UInt32)Commands.Count);
			LengthFieldU32or64 SizeOfCommands = Context.WriteDeferredLength(-2 * sizeof(UInt32), Bits.Num._32); // Size of commands, after this field and the flags field
			
			Context.Write(Flags);
			if (Magic == MH_MAGIC_64)
			{
				Context.Write(Reserved64);
			}

			// Write each command (which may enqueue deferred work)
			foreach (MachLoadCommand Command in Commands)
			{
				Command.Write(Context);
			}
			Context.CommitDeferredField(SizeOfCommands);

			//@TODO: Figure out where this offsetting comes from
			long MainStartPosition = MachHeaderPad;
			Context.WriteZeros(MainStartPosition - Context.Position);

			// Drain deferred work until the file is completely done
			Context.CompleteWritingAndClose();
		}

		public void LoadFromFile(string Filename)
		{
			BinaryReader SR = new BinaryReader(File.OpenRead(Filename));
			ReadingContext Context = new ReadingContext(SR);
			Read(Context);
			SR.Close();
		}

		public void LoadFromBytes(byte[] Data)
		{
			MemoryStream Stream = new MemoryStream(Data, false);
			BinaryReader SR = new BinaryReader(Stream);
			ReadingContext Context = new ReadingContext(SR);
			Read(Context);
			SR.Close();
		}

		public void WriteToFile(string Filename)
		{
			BinaryWriter SW = new BinaryWriter(File.OpenWrite(Filename));
			WritingContext Context = new WritingContext(SW);
			Write(Context);
			SW.Close();
		}
	}

	/// <summary>
	/// Blob for magic CSMAGIC_ENTITLEMENTS
	/// </summary>
	public class EntitlementsBlob : OpaqueBlob
	{
		public EntitlementsBlob()
		{
			MyMagic = CSMAGIC_ENTITLEMENTS;
		}

		public override string ToString()
		{
			return "Entitlements '" + Encoding.ASCII.GetString(MyData, 0, MyData.Length) + "'";
		}

		public static EntitlementsBlob Create(string EntitlementsText)
		{
			EntitlementsBlob Result = new EntitlementsBlob();
			Result.MyData = Encoding.UTF8.GetBytes(EntitlementsText);
			return Result;
		}
	}

	/// <summary>
	/// Blob for magic CSMAGIC_ENTITLEMENTS_DER
	/// </summary>
	public class EntitlementsDerBlob : OpaqueBlob
	{
		public EntitlementsDerBlob()
		{
			MyMagic = CSMAGIC_ENTITLEMENTS_DER;
		}

		public override string ToString()
		{
			return "EntitlementsDER '" + AsnFormatter.BinaryToString(MyData) + "'";
		}

		public static EntitlementsDerBlob Create(string EntitlementsXmlText)
		{
			EntitlementsDerBlob Result = new EntitlementsDerBlob();

			XmlDocument doc = new XmlDocument();
			doc.LoadXml(EntitlementsXmlText);

			XmlNode xmlNode = doc.FirstChild;

			// eat tokens until we see a start 'plist' element.
			while (xmlNode != null && (xmlNode.NodeType != XmlNodeType.Element || xmlNode.Name != "plist"))
			{
				xmlNode = xmlNode.NextSibling;
			}

			Asn1Builder SetBuilder = Asn1Builder.Create();

			if (xmlNode != null)
			{
				// dict node
				xmlNode = xmlNode.FirstChild;
				if (xmlNode != null && xmlNode.NodeType == XmlNodeType.Element && xmlNode.Name == "dict")
				{
					for (int i = 0; i < xmlNode.ChildNodes.Count; i++)
					{
						if (xmlNode.ChildNodes[i].Name == "key")
						{
							Asn1Builder KeyValueBuilder = Asn1Builder.Create();
							KeyValueBuilder.AddUTF8String(xmlNode.ChildNodes[i].InnerText);

							i++;
							if (i >= xmlNode.ChildNodes.Count)
							{
								break;
							}
							if (xmlNode.ChildNodes[i].Name == "string")
							{
								KeyValueBuilder.AddUTF8String(xmlNode.ChildNodes[i].InnerText);
							}
							else if (xmlNode.ChildNodes[i].Name == "true")
							{
								KeyValueBuilder.AddBoolean(true);
							}
							else if (xmlNode.ChildNodes[i].Name == "false")
							{
								KeyValueBuilder.AddBoolean(false);
							}
							else if (xmlNode.ChildNodes[i].Name == "array")
							{
								Asn1Builder ArrayBuilder = Asn1Builder.Create();
								foreach (XmlNode child in xmlNode.ChildNodes[i].ChildNodes)
								{
									if (child.Name == "string")
									{
										ArrayBuilder.AddUTF8String(child.InnerText);
									}
									else if (child.Name == "true")
									{
										ArrayBuilder.AddBoolean(true);
									}
									else if (child.Name == "false")
									{
										ArrayBuilder.AddBoolean(false);
									}
								}

								KeyValueBuilder.AddDerData(ArrayBuilder.GetEncoded());
							}

							SetBuilder.AddDerData(KeyValueBuilder.GetEncoded());
						}
					}
				}
			}

			if (Config.bCodeSignVerbose)
			{
				Console.WriteLine("Generated Entitlements DER as {0}", AsnFormatter.BinaryToString(SetBuilder.GetEncoded(0x31)));
			}

			Result.MyData = SetBuilder.GetEncoded(0x31);

			return Result;
		}
	}

	/// <summary>
	/// Blob for magic CSMAGIC_REQUIREMENTS_TABLE
	/// http://www.opensource.apple.com/source/libsecurity_codesigning/libsecurity_codesigning-36885/lib/requirement.h
	/// 
	/// Currently does not attempt to understand or parse the individual requirements and treats them as opaque data
	/// </summary>
	public class RequirementsBlob : SuperBlob
	{
		public RequirementsBlob()
		{
			MyMagic = CSMAGIC_REQUIREMENTS_TABLE;
		}

		// Valid kinds of hosts that can run the executable
		const int kSecHostRequirementType = 1;

		// Valid guests the executable can run
		const int kSecGuestRequirementType = 2;

		// Explicit requirements
		const int kSecDesignatedRequirementType = 3;

		// Libraries we can link against
		const int kSecLibraryRequirementType = 4;

		public static RequirementsBlob CreateEmpty()
		{
			return new RequirementsBlob();
		}

		protected string KeyToString(int Key)
		{
			switch (Key)
			{
				case kSecHostRequirementType:
					return "kSecHostRequirementType";
				case kSecGuestRequirementType:
					return "kSecGuestRequirementType";
				case kSecDesignatedRequirementType:
					return "kSecDesignatedRequirementType";
				case kSecLibraryRequirementType:
					return "kSecLibraryRequirementType";
				default:
					return String.Format("UnknownKeyType_{0}", Key);
			}
		}

		public override string ToString()
		{
			string Result = "Requirements table:\n";

			foreach (KeyValuePair<UInt32, AbstractBlob> KVP in Table.Slots)
			{
				Result += String.Format("  Requirement[Type: {0}, Magic: 0x{1:X}] = {2}\n",
					KeyToString((int)KVP.Key),
					KVP.Value.MyMagic, KVP.Value.ToString());
			}

			return Result;
		}
	}

	public class RequirementBlob : OpaqueBlob
	{
		const UInt32 kOpUnknown = 1000;
		const UInt32 kOpIdent = 2;
		const UInt32 kOpAnchorHash = 4;
		const UInt32 kOpAnd = 6;
		const UInt32 kOpCertField = 11;
		const UInt32 kOpCertGeneric = 14;
		const UInt32 kOpGenericAnchor = 15;

		public class ExpressionOp
		{
			public UInt32 OpVal = kOpUnknown;

			public ExpressionOp() { }
			public ExpressionOp(UInt32 InOpVal)
			{
				OpVal = InOpVal;
			}

			public virtual void ReadData(ReadingContext SR)
			{ }

			public virtual void WriteData(WritingContext SW)
			{
				SW.Write(OpVal);
			}

			public virtual void UpdateCertificateAndBundle(string CertificateName, string BundleIdentifier)
			{ }

			static public ExpressionOp ReadOperand(ReadingContext SR)
			{
				UInt32 OpVal = SR.ReadUInt32();
				ExpressionOp Op = null;
				switch (OpVal)
				{
					case kOpAnd:
						Op = new AndOp();
						break;

					case kOpIdent:
						Op = new IdentOp();
						break;

					case kOpGenericAnchor:
						Op = new ExpressionOp(OpVal);
						break;

					case kOpCertField:
						Op = new CertFieldOp();
						break;

					case kOpCertGeneric:
						Op = new CertGenericOp();
						break;

					case kOpAnchorHash:
						Op = new AnchorHashOp();
						break;

					default:
						throw new Exception("Unknown Expression Operand: " + OpVal.ToString());
				}

				Op.ReadData(SR);
				return Op;
			}
		};

		class AndOp : ExpressionOp
		{
			public ExpressionOp Op1;
			public ExpressionOp Op2;

			public AndOp()
			{
				OpVal = kOpAnd;
			}

			public override void ReadData(ReadingContext SR)
			{
				Op1 = ExpressionOp.ReadOperand(SR);
				Op2 = ExpressionOp.ReadOperand(SR);
			}

			public override void WriteData(WritingContext SW)
			{
				base.WriteData(SW);
				Op1.WriteData(SW);
				Op2.WriteData(SW);
			}

			public override void UpdateCertificateAndBundle(string CertificateName, string BundleIdentifier)
			{
				Op1.UpdateCertificateAndBundle(CertificateName, BundleIdentifier);
				Op2.UpdateCertificateAndBundle(CertificateName, BundleIdentifier);
			}
		};

		class IdentOp : ExpressionOp
		{
			public string BundleIdentifier;

			public IdentOp()
			{
				OpVal = kOpIdent;
			}

			public override void ReadData(ReadingContext SR)
			{
				UInt32 Count = SR.ReadUInt32();
				BundleIdentifier = SR.ReadFixedASCII((int)Count);
				Count = 4 - Count % 4;
				if (Count > 0 && Count < 4)
					SR.ReadBytes(Count);
			}

			public override void WriteData(WritingContext SW)
			{
				base.WriteData(SW);
				SW.Write(BundleIdentifier.Length);								// bundle identifier length
				SW.WriteFixedASCII(BundleIdentifier, BundleIdentifier.Length);	// bundle identifier string
				int Count = 4 - BundleIdentifier.Length % 4;					// may need to pad to alignment of 4 bytes
				if (Count > 0 && Count < 4)
					SW.WriteZeros(Count);
			}

			public override void UpdateCertificateAndBundle(string InCertificateName, string InBundleIdentifier)
			{
				BundleIdentifier = InBundleIdentifier;
			}
		};

		class CertFieldOp : ExpressionOp
		{
			public int CertificateIndex = 0;
			public string FieldName = "subject.CN";
			struct MatchSuffix
			{
				public UInt32 MatchOp;
				public string CertificateName;
			};
			MatchSuffix MatchOp;

			public CertFieldOp()
			{
				OpVal = kOpCertField;
				MatchOp.MatchOp = kMatchEqual;
			}

			public override void ReadData(ReadingContext SR)
			{
				CertificateIndex = (int)SR.ReadUInt32();					// index in the mobile provision certificate list (always 0 for now)
				UInt32 Count = SR.ReadUInt32();
				FieldName = SR.ReadFixedASCII((int)Count);
				Count = 4 - Count % 4;
				if (Count > 0 && Count < 4)
					SR.ReadBytes(Count);
				MatchOp = new MatchSuffix();
				MatchOp.MatchOp = SR.ReadUInt32();											// must equal
				Count = SR.ReadUInt32();
				MatchOp.CertificateName = SR.ReadFixedASCII((int)Count);
				Count = 4 - Count % 4;
				if (Count > 0 && Count < 4)
					SR.ReadBytes(Count);
			}

			public override void WriteData(WritingContext SW)
			{
				base.WriteData(SW);
				SW.Write(CertificateIndex);													// index in the mobile provision certificate list (always 0 for now)
				SW.Write(FieldName.Length);													// field name length
				SW.WriteFixedASCII(FieldName, FieldName.Length);							// field name to match
				int Count = 4 - FieldName.Length % 4;										// may need to pad to alignment of 4 bytes
				if (Count > 0 && Count < 4)
					SW.WriteZeros(Count);
				SW.Write(MatchOp.MatchOp);													// must equal
				SW.Write(MatchOp.CertificateName.Length);									// length of certficate name
				SW.WriteFixedASCII(MatchOp.CertificateName, MatchOp.CertificateName.Length);// certificate name to match
				Count = 4 - MatchOp.CertificateName.Length % 4;								// may need to pad to alignment of 4 bytes
				if (Count > 0 && Count < 4)
					SW.WriteZeros(Count);
			}

			public override void UpdateCertificateAndBundle(string InCertificateName, string InBundleIdentifier)
			{
				MatchOp.CertificateName = InCertificateName;
			}
		};

		class CertGenericOp : ExpressionOp
		{
			public int OIDIndex = 1;
			byte[] OID = new byte[] { 0x2A, 0x86, 0x48, 0x86, 0xF7, 0x63, 0x64, 0x06, 0x02, 0x01 };
			struct MatchSuffix
			{
				public UInt32 MatchOp;
			};
			MatchSuffix MatchOp;

			public CertGenericOp()
			{
				OpVal = kOpCertGeneric;
				MatchOp.MatchOp = kMatchExists;
			}

			public override void ReadData(ReadingContext SR)
			{
				OIDIndex = (int)SR.ReadUInt32();							// index of the OID value (always 1)
				UInt32 Count = SR.ReadUInt32();
				OID = SR.ReadBytes((int)Count);
				Count = 4 - Count % 4;
				if (Count > 0 && Count < 4)
					SR.ReadBytes(Count);

				MatchOp.MatchOp = SR.ReadUInt32();							// OID must exist
			}

			public override void WriteData(WritingContext SW)
			{
				base.WriteData(SW);
				SW.Write(OIDIndex);											// index of the OID value (always 1)
				SW.Write(OID.Length);										// length of OID
				SW.Write(OID);						// OID to match
				int Count = 4 - OID.Length % 4;										// may need to pad to alignment of 4 bytes
				if (Count > 0 && Count < 4)
					SW.WriteZeros(Count);

				// may need to pad to alignment of 4 bytes
				SW.Write(MatchOp.MatchOp);										// OID must exist
			}
		};

		class AnchorHashOp : ExpressionOp
		{
			byte[] Hash = new byte[] { 0x2A, 0x86, 0x48, 0x86, 0xF7, 0x63, 0x64, 0x06, 0x02, 0x01 };
			int CertificateIndex = -1;
			public AnchorHashOp()
			{
				OpVal = kOpAnchorHash;
			}

			public override void ReadData(ReadingContext SR)
			{
				CertificateIndex = (int)SR.ReadUInt32();
				UInt32 Count = SR.ReadUInt32();
				Hash = SR.ReadBytes((int)Count);
				Count = 4 - Count % 4;
				if (Count > 0 && Count < 4)
					SR.ReadBytes(Count);
			}

			public override void WriteData(WritingContext SW)
			{
				base.WriteData(SW);
				SW.Write(CertificateIndex);											// index of the OID value (always 1)
				SW.Write(Hash.Length);										// length of OID
				SW.Write(Hash);						// OID to match
				int Count = 4 -  Hash.Length % 4;										// may need to pad to alignment of 4 bytes
				if (Count > 0 && Count < 4)
					SW.WriteZeros(Count);
			}
		};

		const int kReqExpression = 1;

		const int kMatchExists = 0;
		const int kMatchEqual = 1;

		string BundleIdentifier = "";
		string CertificateName = "";

		public ExpressionOp Expression = null;

		public RequirementBlob()
		{
			MyMagic = CSMAGIC_REQUIREMENT;
		}

		public static RequirementBlob CreateFromCertificate(X509Certificate2 SigningCert, string Bundle, ExpressionOp OldReq = null)
		{
			RequirementBlob Blob = new RequirementBlob();
			Blob.InitializeFromCert(SigningCert, Bundle, OldReq);
			return Blob;
		}

		protected void InitializeFromCert(X509Certificate2 SigningCert, string Bundle, ExpressionOp OldReq = null)
		{
			BundleIdentifier = Bundle;
			int StartIndex = SigningCert.SubjectName.Name.IndexOf("CN=");
			int EndIndex = -1;
			CertificateName = "";
			if (StartIndex > -1)
			{
				// find the next attribute
				StartIndex += 3;
				char SearchChar = ',';
				if (SigningCert.SubjectName.Name[StartIndex] == '\"')
				{
					// quotes are around the string because of special characters
					StartIndex++;
					SearchChar = '\"';
				}
				EndIndex = SigningCert.SubjectName.Name.IndexOf(SearchChar, StartIndex);
				if (EndIndex == -1)
				{
					// must be at the end, so go to the end
					EndIndex = SigningCert.SubjectName.Name.Length;
					if (SearchChar == '\"')
					{
						EndIndex--;
					}
				}
				// get the string
				CertificateName = SigningCert.SubjectName.Name.Substring(StartIndex, EndIndex - StartIndex);
			}
			if (string.IsNullOrEmpty(CertificateName))
			{
				CertificateName = CryptoAdapter.GetFriendlyNameFromCert(SigningCert);
			}

			// always use the new  requirements when initializing from cert
			Expression = new AndOp();
			(Expression as AndOp).Op1 = new IdentOp();
			((Expression as AndOp).Op1 as IdentOp).BundleIdentifier = BundleIdentifier;
			(Expression as AndOp).Op2 = new AndOp();
			((Expression as AndOp).Op2 as AndOp).Op1 = new ExpressionOp(kOpGenericAnchor);
			((Expression as AndOp).Op2 as AndOp).Op2 = new AndOp();
			(((Expression as AndOp).Op2 as AndOp).Op2 as AndOp).Op1 = new CertFieldOp();
			(((Expression as AndOp).Op2 as AndOp).Op2 as AndOp).Op2 = new CertGenericOp();
		}

		protected override void PackageData(WritingContext SW)
		{
			// update all of the read expressions with the certificate name and bundle identifier
			Expression.UpdateCertificateAndBundle(CertificateName, BundleIdentifier);
			SW.Write(kReqExpression);
			Expression.WriteData(SW);
		}

		protected override void UnpackageData(ReadingContext SR, UInt32 Length)
		{
			if (Length > 0)
			{
				UInt32 ExpressionVal = SR.ReadUInt32();
				if (ExpressionVal == kReqExpression)
				{
					Expression = ExpressionOp.ReadOperand(SR);
				}
			}
		}
	}

	public class CodeSigningTableBlob : SuperBlob
	{
		public CodeSigningTableBlob()
		{
			MyMagic = CSMAGIC_EMBEDDED_SIGNATURE;
		}

		public static CodeSigningTableBlob Create()
		{
			CodeSigningTableBlob Blob = new CodeSigningTableBlob();
			return Blob;
		}
	}

	public abstract class AbstractBlob
	{
		// Key for CodeDirectory blob in superblob
		public const UInt32 CSSLOT_CODEDIRECTORY = 0;

		// Requirements blob
		public const UInt32 CSMAGIC_REQUIREMENT = 0xFADE0C00;

		// Internal requirements
		public const UInt32 CSMAGIC_REQUIREMENTS_TABLE = 0xFADE0C01;

		// Code directory blob (should be in a slot with key = CSSLOT_CODEDIRECTORY)
		public const UInt32 CSMAGIC_CODEDIRECTORY = 0xFADE0C02;

		// Superblob of all signature data (code directory, actual signature, etc...)
		public const UInt32 CSMAGIC_EMBEDDED_SIGNATURE = 0xFADE0CC0;

		// Entitlements
		public const UInt32 CSMAGIC_ENTITLEMENTS = 0xFADE7171;

		// Entitlements DER
		public const UInt32 CSMAGIC_ENTITLEMENTS_DER = 0xFADE7172;

		// Actual signature blob
		public const UInt32 CSMAGIC_CODEDIR_SIGNATURE = 0xFADE0B01;

		
		public UInt32 MyMagic;

		protected abstract void PackageData(WritingContext SW);
		protected abstract void UnpackageData(ReadingContext SR, UInt32 Length);

		public static AbstractBlob CreateFromStream(ReadingContext SR)
		{
			// Read the magic and length (common to all blobs)
			UInt32 Magic = SR.ReadUInt32();
			UInt32 Length = SR.ReadUInt32();

			AbstractBlob Result;

			switch (Magic)
			{
				case CSMAGIC_CODEDIRECTORY:
					Result = new CodeDirectoryBlob();
					break;
				case CSMAGIC_CODEDIR_SIGNATURE:
					Result = new CodeDirectorySignatureBlob();
					break;
				case CSMAGIC_ENTITLEMENTS:
					Result = new EntitlementsBlob();
					break;
				case CSMAGIC_ENTITLEMENTS_DER:
					Result = new EntitlementsDerBlob();
					break;
				case CSMAGIC_REQUIREMENTS_TABLE:
					Result = new RequirementsBlob();
					break;
				case CSMAGIC_EMBEDDED_SIGNATURE:
					Result = new CodeSigningTableBlob();
					break;
				case CSMAGIC_REQUIREMENT:
					Result = new RequirementBlob();
					break;
				default:
					Result = new OpaqueBlob();
					break;
			}
			Result.MyMagic = Magic;
			Result.UnpackageData(SR, Length);

			if (Config.bCodeSignVerbose)
			{
				Console.WriteLine("[Read blob with magic 0x{0:X} and length={1}]\n{2}", Magic, Length, Result.ToString());
			}

			return Result;
		}

		public void Write(WritingContext SW)
		{
			SW.Write(MyMagic);
			
			LengthFieldU32or64 Length = SW.WriteDeferredLength(4, Bits.Num._32);
			PackageData(SW);
			SW.CommitDeferredField(Length);
		}

		/// <summary>
		/// Always uses big endian!!!
		/// Converts a blob back into an array of bytes (does not work if there are any file-absolute offsets, as it serializes starting at 0 of a fake stream)
		/// </summary>
		public byte[] GetBlobBytes()
		{
			MemoryStream MemoryBuffer = new MemoryStream();
			BinaryWriter Writer = new BinaryWriter(MemoryBuffer);
			WritingContext SW = new WritingContext(Writer);
			SW.bStreamLittleEndian = false;

			Write(SW);
			SW.CompleteWritingAndClose();

			return MemoryBuffer.ToArray();
		}
	}

	/// <summary>
	/// 
	/// </summary>
	public class OpaqueBlob : AbstractBlob
	{
		protected byte[] MyData;

		protected override void UnpackageData(ReadingContext SR, UInt32 Length)
		{
			MyData = SR.ReadBytes(Length - (2 * sizeof(UInt32)));
		}

		protected override void PackageData(WritingContext SW)
		{
			SW.Write(MyData);
		}

		public override string ToString()
		{
			return "Opaque blob";//'" + Encoding.ASCII.GetString(MyData, 0, MyData.Length) + "'";
		}
	}

	public abstract class SuperBlob : AbstractBlob
	{
		protected BlobTable Table = new BlobTable();

		public AbstractBlob GetBlobByMagic(UInt32 Magic)
		{
			foreach (KeyValuePair<UInt32, AbstractBlob> KVP in Table.Slots)
			{
				if (KVP.Value.MyMagic == Magic)
				{
					return KVP.Value;
				}
			}
			return null;
		}

		public AbstractBlob GetBlobByKey(UInt32 Key)
		{
			foreach (KeyValuePair<UInt32, AbstractBlob> KVP in Table.Slots)
			{
				if (KVP.Key == Key)
				{
					return KVP.Value;
				}
			}
			return null;
		}

		protected override void UnpackageData(ReadingContext SR, uint Length)
		{
			Table.Read(SR);
		}

		protected override void PackageData(WritingContext SW)
		{
			Table.Write(SW);
		}

		public void Add(UInt32 Key, AbstractBlob Value)
		{
			Table.Slots.Add(new KeyValuePair<UInt32, AbstractBlob>(Key, Value));
		}
	}

	/// <summary>
	/// References:
	///   http://www.opensource.apple.com/source/libsecurity_utilities/libsecurity_utilities-36984/lib/blob.h
	///   http://www.opensource.apple.com/source/libsecurity_utilities/libsecurity_utilities-38535/lib/superblob.h
	///   http://www.opensource.apple.com/source/libsecurity_codesigning/libsecurity_codesigning-36885/lib/cscdefs.h
	/// 
	/// BlobTable:
	///   U32 SlotCount
	///   SerializedSlot Slots[Count]
	///   
	/// SerializedSlot
	///   KeyType Type
	///   U32 Offset	(relative to start of BlobTable)
	///
	/// </summary>
	public class BlobTable
	{
		public List<KeyValuePair<UInt32, AbstractBlob>> Slots = new List<KeyValuePair<UInt32, AbstractBlob>>();

		public AbstractBlob GetByKey(UInt32 Key)
		{
			foreach (KeyValuePair<UInt32, AbstractBlob> KVP in Slots)
			{
				if (KVP.Key == Key)
				{
					return KVP.Value;
				}
			}

			return null;
		}

		/// <summary>
		/// Reads in a data blob, preserving the stream read pointer (Offset is relative to the start of the underlying stream)
		/// </summary>
		AbstractBlob ReadBlob(ReadingContext SR, long Offset)
		{
			SR.PushPositionAndJump(Offset);
			AbstractBlob Result = AbstractBlob.CreateFromStream(SR);
			SR.PopPosition();

			return Result;
		}

		public void Write(WritingContext SW)
		{
			// Magic (written in the outer)
			// Length (written in the outer)
			// SlotCount
			// Slot SlotTable[SlotCount]
			//   Each slot is Key, Offset
			// <Slot-referenced data>

			//WritingContext.OffsetFieldU32or64[] Offsets = new WritingContext.OffsetFieldU32or64[Slots.Count];
			long StartingPosition = SW.Position - (2 * sizeof(UInt32));

			// Start a phase for writing out the superblob data
			SW.CreateNewPhase();

			// Write the slot table, queuing up the individual slot writes
			SW.Write((UInt32)Slots.Count);

			foreach (KeyValuePair<UInt32, AbstractBlob> Slot in Slots)
			{
				SW.Write(Slot.Key);

				OffsetFieldU32or64 Offset = SW.WriteDeferredOffsetFrom(StartingPosition, Bits.Num._32);
				KeyValuePair<UInt32, AbstractBlob> LocalSlot = Slot;

				SW.CurrentPhase.PendingWrites.Enqueue(delegate(WritingContext Context)
					{
						if (Config.bCodeSignVerbose)
						{
							Console.WriteLine("Writing a slot.  Offset={0}, SlotData={1}", Offset.WritePoint, LocalSlot.ToString());
						}
						SW.CommitDeferredField(Offset);
						LocalSlot.Value.Write(Context);
					});
			}

			// Force evaluation of the slots
			SW.ProcessEntirePhase();
		}

		public void Read(ReadingContext SR)
		{
			long BaseOffset = SR.Position - (2 * sizeof(UInt32));

			UInt32 SlotCount = SR.ReadUInt32();

			for (UInt32 i = 0; i < SlotCount; ++i)
			{
				// Read a slot
				UInt32 Key = SR.ReadUInt32();
				UInt32 Offset = SR.ReadUInt32();

				// Read it's associated blob
				AbstractBlob Blob = ReadBlob(SR, BaseOffset + Offset);

				// Add it to the slot table
				Slots.Add(new KeyValuePair<UInt32, AbstractBlob>(Key, Blob));
			}
		}
	}

	
	public class CodeDirectorySignatureBlob : OpaqueBlob
	{
		public CodeDirectorySignatureBlob()
		{
			MyMagic = CSMAGIC_CODEDIR_SIGNATURE;
		}

		public static CodeDirectorySignatureBlob Create()
		{
			return new CodeDirectorySignatureBlob();
		}

		public void DebugCMSData(byte[] Data, string Prefix)
		{
			SignedCms CMS = new SignedCms();

			CMS.Decode(Data);
			int Version = CMS.Version;

			Console.WriteLine("CMS Blob {0} Info", Prefix);
			Console.WriteLine("  Version = {0}", Version);
			foreach (SignerInfo si in CMS.SignerInfos)
			{
				Console.WriteLine("  SignerInfo {");
				Console.WriteLine("    Cert issued by {0}", si.Certificate.Issuer);
				foreach (CryptographicAttributeObject Attr in si.SignedAttributes)
				{
					foreach (AsnEncodedData AttrData in Attr.Values)
					{
						Pkcs9SigningTime SignTime = AttrData as Pkcs9SigningTime;
						Pkcs9ContentType CType = AttrData as Pkcs9ContentType;
						Pkcs9MessageDigest Digest = AttrData as Pkcs9MessageDigest;

						if (SignTime != null)
							Console.WriteLine("    SignedTime {0:O}", SignTime.SigningTime);
						if (CType != null)
							Console.WriteLine("    ContentType {0}", CType.ContentType.FriendlyName);
						if (Digest != null)
							Console.WriteLine("    MessageDigest h'{0}'", BitConverter.ToString(Digest.RawData).Replace("-", ""));
					}
				}
				Console.WriteLine("    }");
			}
			
			Console.WriteLine("  Inner content: h'{0}'", BitConverter.ToString(CMS.ContentInfo.Content).Replace("-", ""));
		}

		/// <summary>
		/// Populates this CMS blob with the data from signing a code directory
		/// </summary>
		public void SignCodeDirectory(X509Certificate2 SigningCert, DateTime SigningTime, CodeDirectoryBlob CodeDirectory)
		{
			// Create a signer
			CmsSigner Signer = new CmsSigner(SigningCert);
			Signer.IncludeOption = X509IncludeOption.WholeChain;
			Signer.SignerIdentifierType = SubjectIdentifierType.IssuerAndSerialNumber;
			Signer.DigestAlgorithm = new Oid(CryptoConfig.MapNameToOID("sha256"), "sha256");

			// A Pkcs9ContentType and Pkcs9MessageDigest will automatically be added, and it fails to
			// compute a signature if they are added manually, so only the signing time needs to be added
			Signer.SignedAttributes.Add(new Pkcs9SigningTime(SigningTime));

			// Sign the data (in a detached manner, so only the digest of the CodeDirectory is
			// stored in the CMS blob and not the whole CodeDirectory blob)
			bool bDetached = true;
			bool bSilent = true;
			ContentInfo CodeDirContentInfo = new ContentInfo(CodeDirectory.GetBlobBytes());
			SignedCms CMS = new SignedCms(CodeDirContentInfo, bDetached);
			CMS.ComputeSignature(Signer, bSilent);

			MyData = CMS.Encode();
		}

		public override string ToString()
		{
			DebugCMSData(this.MyData, "CMS blob ToString");
			return "CMS Blob";
		}
	}

	public class CodeDirectoryBlob : AbstractBlob
	{
		public const UInt32 cVersion2 = 0x20200;
		public const UInt32 cVersion3 = 0x20400;

		public UInt32 Version;
		UInt32 Flags;

		//UInt32 HashOffset;
		byte[] Hashes;

		//UInt32 IdentifierStringOffset;
		string Identifier;
		
		UInt32 SpecialSlotCount;
		UInt32 CodeSlotCount;
		UInt32 MainImageSignatureLimit;
		byte BytesPerHash;
		byte HashType;
		byte Spare1;
		byte LogPageSize;
		UInt32 Spare2;
		UInt32 ScatterCount;

		string Team;

		UInt32 Spare3;
		UInt64 CodeLimit64;

		UInt64 ExecSegBase;
		UInt64 ExecSegLimit;
		UInt64 ExecSegFlags;

		// Special slot index for Info.plist
		public const int cdInfoSlot = 1;

		// Special slot index for internal requirements
		public const int cdRequirementsSlot = 2;

		// Special slot index for the resource directory
		public const int cdResourceDirSlot = 3;

		// Special slot index for the application
		public const int cdApplicationSlot = 4;

		// Special slot index for embedded entitlements
		public const int cdEntitlementSlot = 5;

		// Special slot index 6, unknown - all 0's in practice
		public const int cdUnknown6Slot = 6;

		// Special slot index for embedded der entitlements
		public const int cdDerEntitlementSlot = 7;

		// Number of special slot indexes
		public const int cdSlotMax = cdDerEntitlementSlot;

		/// <summary>
		/// Hash provider (SHA1)
		/// </summary>
		protected SHA256CryptoServiceProvider HashProvider2 = new SHA256CryptoServiceProvider();

		protected override void UnpackageData(ReadingContext SR, UInt32 Length)
		{
			long StartOfBlob = SR.Position - sizeof(UInt32) * 2;

			// References:
			//  https://llvm.org/doxygen/BinaryFormat_2MachO_8h_source.html
			//  https://github.com/apple/darwin-xnu/blob/main/osfmk/kern/cs_blobs.h
			Version = SR.ReadUInt32();
			Flags = SR.ReadUInt32();
			UInt32 HashOffset = SR.ReadUInt32();
			UInt32 IdentifierStringOffset = SR.ReadUInt32();
			SpecialSlotCount = SR.ReadUInt32();
			CodeSlotCount = SR.ReadUInt32();
			MainImageSignatureLimit = SR.ReadUInt32();
			BytesPerHash = SR.ReadByte();
			HashType = SR.ReadByte();
			Spare1 = SR.ReadByte();
			LogPageSize = SR.ReadByte();
			Spare2 = SR.ReadUInt32();
			ScatterCount = SR.ReadUInt32();
			UInt32 TeamStringOffset = SR.ReadUInt32();
			Spare3 = SR.ReadUInt32();
			CodeLimit64 = SR.ReadUInt64();
			ExecSegBase = SR.ReadUInt64();
			ExecSegLimit = SR.ReadUInt64();
			ExecSegFlags = SR.ReadUInt64();

			// Read the identifier string
			SR.PushPositionAndJump(StartOfBlob + IdentifierStringOffset);
			Identifier = SR.ReadASCIIZ();
			SR.PopPosition();
			// Read the team string
			SR.PushPositionAndJump(StartOfBlob + TeamStringOffset);
			Team = SR.ReadASCIIZ();
			SR.PopPosition();

			// Read the hashes
			long TotalNumHashes = SpecialSlotCount + CodeSlotCount;
			Hashes = new byte[TotalNumHashes * BytesPerHash];

			SR.PushPositionAndJump(StartOfBlob + HashOffset - BytesPerHash * SpecialSlotCount);
			for (long i = 0; i < TotalNumHashes; ++i)
			{
				byte[] Hash = SR.ReadBytes(BytesPerHash);
				Array.Copy(Hash, 0, Hashes, i * BytesPerHash, BytesPerHash);
			}
			SR.PopPosition();


			if (Config.bCodeSignVerbose)
			{
				PrintHash("Info:", cdSlotMax - cdInfoSlot);
				PrintHash("Requirements:", cdSlotMax - cdRequirementsSlot);
				PrintHash("ResourceDir:", cdSlotMax - cdResourceDirSlot);
				PrintHash("Application:", cdSlotMax - cdApplicationSlot);
				PrintHash("Entitlements:", cdSlotMax - cdEntitlementSlot);
			}
		}

		void PrintHash(string Prefix, int SlotIndex)
		{
			Console.WriteLine("{0} {1}", Prefix, BitConverter.ToString(Hashes, SlotIndex * BytesPerHash, BytesPerHash).Replace("-", ""));
		}

		protected override void PackageData(WritingContext SW)
		{
			long StartPos = SW.Position - (2 * sizeof(UInt32));

			SW.Write(Version);
			SW.Write(Flags);

			// The hash offset is weird, it points to the first code page hash, not the start of the hashes array...
			OffsetFieldU32or64 HashOffset = SW.WriteDeferredOffsetFrom(StartPos - (BytesPerHash * SpecialSlotCount), Bits.Num._32);
			OffsetFieldU32or64 IdentifierStringOffset = SW.WriteDeferredOffsetFrom(StartPos, Bits.Num._32);
			
			SW.Write(SpecialSlotCount);
			SW.Write(CodeSlotCount);
			SW.Write(MainImageSignatureLimit);
			
			SW.Write(BytesPerHash);
			SW.Write(HashType);
			SW.Write(Spare1);
			SW.Write(LogPageSize);
			
			SW.Write(Spare2);
			SW.Write(ScatterCount);

			OffsetFieldU32or64 TeamStringOffset = SW.WriteDeferredOffsetFrom(StartPos, Bits.Num._32);
			SW.Write(Spare3);

			SW.Write(CodeLimit64);
			SW.Write(ExecSegBase);
			SW.Write(ExecSegLimit);
			SW.Write(ExecSegFlags);

			// Write the identifier
			SW.CommitDeferredField(IdentifierStringOffset);
			byte[] IdentifierOutput = Utilities.CreateASCIIZ(Identifier);
			SW.Write(IdentifierOutput);

			// Write the team identifier
			SW.CommitDeferredField(TeamStringOffset);
			byte[] TeamOutput = Utilities.CreateASCIIZ(Team);
			SW.Write(TeamOutput);
		
			// Write the hashes
			SW.CommitDeferredField(HashOffset);
			SW.Write(Hashes);
		}

		public CodeDirectoryBlob()
		{
			MyMagic = CSMAGIC_CODEDIRECTORY;
		}

		/// <summary>
		/// The page size in bytes
		/// </summary>
		public int PageSize
		{
			get { return 1 << LogPageSize; }
		}

		/// <summary>
		/// Generates a hash for one of the special slots and copies it into the hash array
		/// </summary>
		public void GenerateSpecialSlotHash(int SpecialSlotIndex, byte[] SourceData)
		{
			byte[] Hash = HashProvider2.ComputeHash(SourceData);
			Array.Copy(
				Hash, 0,
				Hashes, (SpecialSlotCount - SpecialSlotIndex) * BytesPerHash,
				BytesPerHash);
		}


		/// <summary>
		/// Generates an empty hash (all zeros) for a special slot that isn't used (only cdApplicationSlot gets this treatment)
		/// </summary>
		/// <param name="SpecialSlotIndex"></param>
		public void GenerateSpecialSlotHash(int SpecialSlotIndex)
		{
			for (int i = 0; i < BytesPerHash; ++i)
			{
				Hashes[(cdSlotMax - SpecialSlotIndex) * BytesPerHash + i] = 0;
			}
		}

		public static CodeDirectoryBlob Create(string ApplicationID, string TeamID, int SignedFileLength, UInt64 InExecSegBase, UInt64 InExecSegLimit)
		{
			CodeDirectoryBlob Blob = new CodeDirectoryBlob();
			Blob.Allocate(ApplicationID, TeamID, SignedFileLength, InExecSegBase, InExecSegLimit);

			return Blob;
		}

		public void Allocate(string ApplicationID, string TeamID, int SignedFileLength, UInt64 InExecSegBase, UInt64 InExecSegLimit)
		{
			Identifier = ApplicationID;
			Team = TeamID;

			Version = cVersion3;
			Flags = 0;
			Spare1 = 0;
			Spare2 = 0;
			Spare3 = 0;
			ScatterCount = 0;

			CodeLimit64 = 0;

			ExecSegBase = InExecSegBase;
			ExecSegLimit = InExecSegLimit;
			ExecSegFlags = 17;

			// 4 KB pages
			LogPageSize = 12;
			int PageSize = 1 << LogPageSize;

			// 32 byte SHA256 hashes
			HashType = 2;
			BytesPerHash = (byte)(HashProvider2.HashSize / 8);
			Debug.Assert(BytesPerHash == 32);

			// Allocate space for the hashes
			MainImageSignatureLimit = (UInt32)SignedFileLength;
			SpecialSlotCount = cdSlotMax;
			CodeSlotCount = (uint)((MainImageSignatureLimit + PageSize - 1) / PageSize);
			Hashes = new byte[(SpecialSlotCount + CodeSlotCount) * BytesPerHash];
		}
		
		/// <summary>
		/// Calculates the hashes for every 4 KB of the main executable
		/// </summary>
		public void ComputeImageHashes(byte[] SignedFileData)
		{
			// Fill out the executable hash
			for (int i = 0; i < CodeSlotCount; ++i)
			{
				int StartOffset = i * PageSize;
				int LengthRemaining = (int)MainImageSignatureLimit - StartOffset;
				byte[] PageHash = HashProvider2.ComputeHash(SignedFileData, StartOffset, Math.Min(LengthRemaining, PageSize)); 
				Array.Copy(PageHash, 0, Hashes, (SpecialSlotCount + i) * BytesPerHash, BytesPerHash);
			}
		}

		public override string ToString()
		{
			return String.Format(
				"\n  CodeDirectory\n    v{0:X}\n    flags={1}\n    Ident: {2}\n    with {3}+{4} slots",
					Version, Flags, Identifier,
					SpecialSlotCount, CodeSlotCount) +

				   String.Format(
				   "\n    SigLimit 0x{0:X}\n    Hash(Len={1} Type={2} Page={3})\n    Spares={4} {5}\n    ScatterCount={6}\n\n",
				   MainImageSignatureLimit, BytesPerHash, HashType, 1 << LogPageSize, Spare1, Spare2, ScatterCount);
		}
	}

	/// <summary>
	/// Payload for LC_CODE_SIGNATURE
	/// </summary>
	class MachLoadCommandCodeSignature : MachLoadCommandLinkEditBlob
	{
		public CodeSigningTableBlob Payload;

		public override string ToString()
		{
			return "Code Signature " + base.ToString();
		}

		protected override void UnpackageData(ReadingContext SR, int CommandSize)
		{
			base.UnpackageData(SR, CommandSize);

			SR.PushPositionAndJump(BlobFileOffset);
			SR.bStreamLittleEndian = false;
			long SavedPosition = SR.Position;

			// Parse the blob
			Payload = AbstractBlob.CreateFromStream(SR) as CodeSigningTableBlob;

			if (Config.bCodeSignVerbose)
			{
				Console.WriteLine(Payload.ToString());
			}

			//SR.VerifyStreamPosition(SavedPosition, BlobFileSize);
			
			SR.PopPosition();
			SR.bStreamLittleEndian = true;
		}


		/// <summary>
		/// Patches the the blob length and offset of the code signing blob in an existing file
		/// </summary>
		public void PatchPositionAndSize(WritingContext SW, UInt32 NewOffset, UInt32 NewLength)
		{
			Debug.Assert(StartingLoadOffset >= 0);

			if (Config.bCodeSignVerbose)
			{
				Console.WriteLine("ZZZZZZZZZZZZZZZZ");
				Console.WriteLine(" Blob offset from 0x{0:X} to 0x{1:X} (delta {2})", BlobFileOffset, NewOffset, (long)NewOffset - (long)BlobFileOffset);
				Console.WriteLine(" Blob size from 0x{0:X} to 0x{1:X} (delta {2})", BlobFileSize, NewLength, (long)NewLength - (long)BlobFileSize);
				Console.WriteLine("ZZZZZZZZZZZZZZZZ");
			}

			BlobFileOffset = NewOffset;
			BlobFileSize = NewLength;

			long PatchOffset = StartingLoadOffset + (2 * sizeof(UInt32)); // Command, CommandLength

			SW.PushPositionAndJump(PatchOffset);
			SW.Write(BlobFileOffset);
			SW.Write(BlobFileSize);
			SW.PopPosition();
		}
	}

	/// <summary>
	/// Minimal implementation of the Fat Binary object file format
	/// See: https://developer.apple.com/library/mac/documentation/DeveloperTools/Conceptual/MachORuntime/Reference/reference.html#//apple_ref/doc/uid/20001298-154889

	public class FatBinaryArch
	{
		public UInt32 CpuType;
		public UInt32 CpuSubType;
		public UInt32 Offset;			// Offset to the beginning of the data for this CPU.
		public UInt32 Size;				// Size of the data for this CPU.
		public UInt32 Align;
	}

	public class FatBinaryFile
	{
		public UInt32 Magic;
		public UInt32 NumArchs;

		// Was this really a fat binary, or are we just a wrapper for the MachObjectFile?
		public bool bIsFatBinary;

		public List<FatBinaryArch> Archs;

		public List<MachObjectFile> MachObjectFiles = new List<MachObjectFile>();
	
		// Values for Magic
		public const UInt32 FAT_MAGIC = 0xCAFEBABE;
		public const UInt32 FAT_CIGAM = 0xBEBAFECA; // For little Endian.

		protected void Read(ReadingContext SR)
		{
			// Read the header to see if it is a FAT Binary or not.
			SR.bStreamLittleEndian = false;
			Magic = SR.ReadUInt32();
			bIsFatBinary = (Magic == FAT_MAGIC);

			if (bIsFatBinary)
			{
				Archs =  new List<FatBinaryArch>();
				NumArchs = SR.ReadUInt32();

				for (int ArchIdx = 0; ArchIdx < NumArchs; ArchIdx++)
				{
					SR.bStreamLittleEndian = false;

					FatBinaryArch Arch = new FatBinaryArch();

					Arch.CpuType = SR.ReadUInt32();
					Arch.CpuSubType = SR.ReadUInt32();
					Arch.Offset = SR.ReadUInt32();
					Arch.Size = SR.ReadUInt32();
					Arch.Align = SR.ReadUInt32();

					Archs.Add(Arch);

					MachObjectFile Exe = new MachObjectFile();

					SR.bStreamLittleEndian = true;
					SR.OpenFatArchiveAt(Arch.Offset);
					Exe.Read(SR);
					SR.CloseFatArchive();

					MachObjectFiles.Add(Exe);
				}
				SR.bStreamLittleEndian = true;
			}
			else
			{
				SR.bStreamLittleEndian = true;
				MachObjectFile Exe = new MachObjectFile();
				SR.Position = 0;
				Exe.Read(SR);
				MachObjectFiles.Add(Exe);
			}
		}

		protected void Write(WritingContext Context)
		{
			if (bIsFatBinary)
			{
				// Write the header
				Context.Write(Magic);
				Context.Write(NumArchs);

				Context.PushPosition();

				foreach (FatBinaryArch Arch in Archs)
				{
					Context.Write(Arch.CpuType);
					Context.Write(Arch.CpuSubType);
					Context.Write(Arch.Offset);
					Context.Write(Arch.Size);
					Context.Write(Arch.Align);
				}

				int FileIdx = 0;
				foreach (MachObjectFile MachFile in MachObjectFiles)
				{
					Archs[FileIdx].Offset = Convert.ToUInt32(Context.Position);
					MachFile.Write(Context);
					Archs[FileIdx].Size = Convert.ToUInt32(Context.Position) - Archs[FileIdx].Offset;
					FileIdx++;
				}

				Context.PopPosition();

				// Write updated header.
				foreach (FatBinaryArch Arch in Archs)
				{
					Context.Write(Arch.CpuType);
					Context.Write(Arch.CpuSubType);
					Context.Write(Arch.Offset);
					Context.Write(Arch.Size);
					Context.Write(Arch.Align);
				}
			}
			else
			{
				// Should only be one...
				MachObjectFiles[0].Write(Context);
			}
		}

		public void WriteHeader(ref byte[] OutputData, uint Offset)
		{
			if (bIsFatBinary)
			{
				// Write the header
				byte[] Data = BitConverter.GetBytes(Magic);
				if (BitConverter.IsLittleEndian)
				{
					Array.Reverse(Data);
				}
				Data.CopyTo(OutputData, Offset);
				Offset += sizeof(UInt32);

				Data = BitConverter.GetBytes(NumArchs);
				if (BitConverter.IsLittleEndian)
				{
					Array.Reverse(Data);
				}
				Data.CopyTo(OutputData, Offset);
				Offset += sizeof(UInt32);

				foreach (FatBinaryArch Arch in Archs)
				{
					Data = BitConverter.GetBytes(Arch.CpuType);
					if (BitConverter.IsLittleEndian)
					{
						Array.Reverse(Data);
					}
					Data.CopyTo(OutputData, Offset);
					Offset += sizeof(UInt32);

					Data = BitConverter.GetBytes(Arch.CpuSubType);
					if (BitConverter.IsLittleEndian)
					{
						Array.Reverse(Data);
					}
					Data.CopyTo(OutputData, Offset);
					Offset += sizeof(UInt32);

					Data = BitConverter.GetBytes(Arch.Offset);
					if (BitConverter.IsLittleEndian)
					{
						Array.Reverse(Data);
					}
					Data.CopyTo(OutputData, Offset);
					Offset += sizeof(UInt32);

					Data = BitConverter.GetBytes(Arch.Size);
					if (BitConverter.IsLittleEndian)
					{
						Array.Reverse(Data);
					}
					Data.CopyTo(OutputData, Offset);
					Offset += sizeof(UInt32);

					Data = BitConverter.GetBytes(Arch.Align);
					if (BitConverter.IsLittleEndian)
					{
						Array.Reverse(Data);
					}
					Data.CopyTo(OutputData, Offset);
					Offset += sizeof(UInt32);
				}
			}
		}

		public void LoadFromFile(string Filename)
		{
			BinaryReader SR = new BinaryReader(File.OpenRead(Filename));
			ReadingContext Context = new ReadingContext(SR);
			Read(Context);
			SR.Close();
		}

		public void LoadFromBytes(byte[] Data)
		{
			MemoryStream Stream = new MemoryStream(Data, false);
			BinaryReader SR = new BinaryReader(Stream);
			ReadingContext Context = new ReadingContext(SR);
			Read(Context);
			SR.Close();
		}

		public void WriteToFile(string Filename)
		{
			BinaryWriter SW = new BinaryWriter(File.OpenWrite(Filename));
			WritingContext Context = new WritingContext(SW);
			Write(Context);
			SW.Close();
		}
	}
}
