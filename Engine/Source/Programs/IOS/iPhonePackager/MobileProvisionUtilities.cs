/**
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections.Generic;
using System.Text;
using System.Security.Cryptography.X509Certificates;
using System.Security.Cryptography;
using System.IO;
using System.Diagnostics;
using System.Xml;
using System.Globalization;
using System.Linq;

namespace iPhonePackager
{
    /// <summary>
    /// Represents the salient parts of a mobile provision, wrt. using it for code signing
    /// </summary>
    public class MobileProvision
    {
        public object Tag;

        public string ApplicationIdentifierPrefix = null;
        public string ApplicationIdentifier = null;
        public List<X509Certificate2> DeveloperCertificates = new List<X509Certificate2>();
        public List<string> ProvisionedDeviceIDs;
        public string ProvisionName;
        public bool bDebug;
        public Utilities.PListHelper Data;
        public DateTime CreationDate;
        public DateTime ExpirationDate;
        public string FileName;
        public string UUID;
        public string Platform;

        public static void CacheMobileProvisions()
        {
            Program.Log("Caching provisions");
            if (!Directory.Exists(Config.ProvisionDirectory))
            {
                Program.Log("Provision Folder {0} doesn't exist, creating..", Config.ProvisionDirectory);
                Directory.CreateDirectory(Config.ProvisionDirectory);
            }

			List<string> ProvisionCopySrcDirectories = new List<string>();

			// Paths for provisions under game directory 
			if (!String.IsNullOrEmpty(Config.ProjectFile))
			{
				ProvisionCopySrcDirectories.Add(Path.GetDirectoryName(Config.ProjectFile) + "/Build/" + Config.OSString + "/");
				ProvisionCopySrcDirectories.Add(Path.GetDirectoryName(Config.ProjectFile) + "/Restricted/NoRedist/Build/" + Config.OSString + "/");
				ProvisionCopySrcDirectories.Add(Path.GetDirectoryName(Config.ProjectFile) + "/Restricted/NotForLicensees/Build/" + Config.OSString + "/");
			}

			// Paths for provisions under the Engine directory
			string OverrideProvisionDirectory = Environment.GetEnvironmentVariable("ProvisionDirectory");
			if(!String.IsNullOrEmpty(OverrideProvisionDirectory))
			{
				ProvisionCopySrcDirectories.Add(OverrideProvisionDirectory);
			}
			else
			{
				ProvisionCopySrcDirectories.Add(Config.EngineBuildDirectory);
				ProvisionCopySrcDirectories.Add(Config.EngineDirectory + "/Restricted/NoRedist/Build/" + Config.OSString + "/");
				ProvisionCopySrcDirectories.Add(Config.EngineDirectory + "/Restricted/NotForLicensees/Build/" + Config.OSString + "/");
			}

			// Copy all provisions from the above paths to the library
			foreach (string ProvisionCopySrcDirectory in ProvisionCopySrcDirectories)
			{
				if (Directory.Exists(ProvisionCopySrcDirectory))
				{
					Program.Log("Finding provisions in {0}", ProvisionCopySrcDirectory);
					foreach (string Provision in Directory.EnumerateFiles(ProvisionCopySrcDirectory, "*.mobileprovision", SearchOption.AllDirectories))
					{
						string TargetFile = Config.ProvisionDirectory + Path.GetFileName(Provision);
						if (!File.Exists(TargetFile) || File.GetLastWriteTime(TargetFile) < File.GetLastWriteTime(Provision))
						{
							FileInfo DestFileInfo;
							if (File.Exists(TargetFile))
							{
								DestFileInfo = new FileInfo(TargetFile);
								DestFileInfo.Attributes = DestFileInfo.Attributes & ~FileAttributes.ReadOnly;
							}
							Program.Log("  Copying {0} -> {1}", Provision, TargetFile);
							File.Copy(Provision, TargetFile, true);
							DestFileInfo = new FileInfo(TargetFile);
							DestFileInfo.Attributes = DestFileInfo.Attributes & ~FileAttributes.ReadOnly;
							if (!File.Exists(TargetFile))
							{
								Program.Log("ERROR: Failed to copy {0} -> {1}", Provision, TargetFile);
							}
						}
						else
						{
							Program.Log("  Not copying {0} as {1} already exists and is not older", Provision, TargetFile);
						}
					}
				}
			}
        }

		public static void CleanMobileProvisions()
		{
			if (!Directory.Exists(Config.ProvisionDirectory))
			{
				Program.Log("Provision Folder {0} doesn't exist, nothing to do.", Config.ProvisionDirectory);
			}
			else
			{
				Program.Log("Cleaning out contents of  Provision Folder {0}", Config.ProvisionDirectory);
				foreach (string Provision in Directory.GetFiles(Config.ProvisionDirectory))
				{
					File.Delete(Provision);
				}
			}
		}


		public static string FindCompatibleProvision(string CFBundleIdentifier, out bool bNameMatch, bool bCheckCert = true, bool bCheckIdentifier = true, bool bCheckDistro = true)
        {
            bNameMatch = false;

            // remap the gamename if necessary
            string GameName = Program.GameName;
            if (GameName == "UnrealGame")
            {
                if (Config.ProjectFile.Length > 0)
                {
                    GameName = Path.GetFileNameWithoutExtension(Config.ProjectFile);
                }
            }

            // ensure the provision directory exists
            if (!Directory.Exists(Config.ProvisionDirectory))
            {
                Directory.CreateDirectory(Config.ProvisionDirectory);
            }

            if (Config.bProvision)
            {
                if (File.Exists(Config.ProvisionDirectory + "/" + Config.Provision))
                {
                    return Config.ProvisionDirectory + "/" + Config.Provision;
                }
            }

            #region remove after we provide an install mechanism
            CacheMobileProvisions();
            #endregion

            // cache the provision library
            Dictionary<string, MobileProvision> ProvisionLibrary = new Dictionary<string, MobileProvision>();
            foreach (string Provision in Directory.EnumerateFiles(Config.ProvisionDirectory, "*.mobileprovision"))
            {
                MobileProvision p = MobileProvisionParser.ParseFile(Provision);
                ProvisionLibrary.Add(Provision, p);
                if (p.FileName.Contains(p.UUID) && !File.Exists(Path.Combine(Config.ProvisionDirectory, "UE4_" + p.UUID + ".mobileprovision")))
                {
                    File.Copy(Provision, Path.Combine(Config.ProvisionDirectory, "UE4_" + p.UUID + ".mobileprovision"));
                    p = MobileProvisionParser.ParseFile(Path.Combine(Config.ProvisionDirectory, "UE4_" + p.UUID + ".mobileprovision"));
                    ProvisionLibrary.Add(Path.Combine(Config.ProvisionDirectory, "UE4_" + p.UUID + ".mobileprovision"), p);
                }
            }

            Program.Log("Searching for mobile provisions that match the game '{0}' (distribution: {3}) with CFBundleIdentifier='{1}' in '{2}'", GameName, CFBundleIdentifier, Config.ProvisionDirectory, Config.bForDistribution);

            // first sort all profiles so we look at newer ones first.
            IEnumerable<string> ProfileKeys = ProvisionLibrary.Select(KV => KV.Key)
                .OrderByDescending(K => ProvisionLibrary[K].CreationDate)
                .ToArray();

            // check the cache for a provision matching the app id (com.company.Game)
            // First checking for a contains match and then for a wildcard match
            for (int Phase = -1; Phase < 3; ++Phase)
            {
                if (Phase == -1 && string.IsNullOrEmpty(Config.ProvisionUUID))
                {
                    continue;
                }
                foreach (string Key in ProfileKeys)
                {
                    string DebugName = Path.GetFileName(Key);
                    MobileProvision TestProvision = ProvisionLibrary[Key];

                    // make sure the file is not managed by Xcode
                    if (Path.GetFileName(TestProvision.FileName).ToLower().Equals(TestProvision.UUID.ToLower() + ".mobileprovision"))
                    {
                        continue;
                    }

                    Program.LogVerbose("  Phase {0} considering provision '{1}' named '{2}'", Phase, DebugName, TestProvision.ProvisionName);

                    if (TestProvision.ProvisionName == "iOS Team Provisioning Profile: " + CFBundleIdentifier)
                    {
                        Program.LogVerbose("  Failing as provisioning is automatic");
                        continue;
                    }

                    // check to see if the platform is the same as what we are looking for
                    if (!string.IsNullOrEmpty(TestProvision.Platform) && TestProvision.Platform != Config.OSString && !string.IsNullOrEmpty(Config.OSString))
                    {
                        //Program.LogVerbose("  Failing platform {0} Config: {1}", TestProvision.Platform, Config.OSString);
                        continue;
                    }

                    // Validate the name
                    bool bPassesNameCheck = false;
                    if (Phase == -1)
                    {
                        bPassesNameCheck = TestProvision.UUID == Config.ProvisionUUID;
                        bNameMatch = bPassesNameCheck;
                    }
                    else if (Phase == 0)
                    {
                        bPassesNameCheck = TestProvision.ApplicationIdentifier.Substring(TestProvision.ApplicationIdentifierPrefix.Length + 1) == CFBundleIdentifier;
                        bNameMatch = bPassesNameCheck;
                    }
                    else if (Phase == 1)
                    {
                        if (TestProvision.ApplicationIdentifier.Contains("*"))
                        {
                            string CompanyName = TestProvision.ApplicationIdentifier.Substring(TestProvision.ApplicationIdentifierPrefix.Length + 1);
                            if (CompanyName != "*")
                            {
                                CompanyName = CompanyName.Substring(0, CompanyName.LastIndexOf("."));
                                bPassesNameCheck = CFBundleIdentifier.StartsWith(CompanyName);
                            }
                        }
                    }
                    else
                    {
                        if (TestProvision.ApplicationIdentifier.Contains("*"))
                        {
                            string CompanyName = TestProvision.ApplicationIdentifier.Substring(TestProvision.ApplicationIdentifierPrefix.Length + 1);
                            bPassesNameCheck = CompanyName == "*";
                        }
                    }
                    if (!bPassesNameCheck && bCheckIdentifier)
                    {
                        Program.LogVerbose("  .. Failed phase {0} name check (provision app ID was {1})", Phase, TestProvision.ApplicationIdentifier);
                        continue;
                    }

                    if (Config.bForDistribution)
                    {
                        // Check to see if this is a distribution provision. get-task-allow must be false for distro profiles.
                        // TestProvision.ProvisionedDeviceIDs.Count==0 is not a valid check as ad-hoc distro profiles do list devices.
                        bool bDistroProv = !TestProvision.bDebug;
                        if (!bDistroProv)
                        {
                            Program.LogVerbose("  .. Failed distribution check (mode={0}, get-task-allow={1}, #devices={2})", Config.bForDistribution, TestProvision.bDebug, TestProvision.ProvisionedDeviceIDs.Count);
                            continue;
                        }
                    }
                    else
                    {
                        if (bCheckDistro)
                        {
                            bool bPassesDebugCheck = TestProvision.bDebug;
                            if (!bPassesDebugCheck)
                            {
                                Program.LogVerbose("  .. Failed debugging check (mode={0}, get-task-allow={1}, #devices={2})", Config.bForDistribution, TestProvision.bDebug, TestProvision.ProvisionedDeviceIDs.Count);
                                continue;
                            }
                        }
                        else
                        {
                            if (!TestProvision.bDebug)
                            {
                                Config.bForceStripSymbols = true;
                            }
                        }
                    }

                    // Check to see if the provision is in date
                    DateTime CurrentUTCTime = DateTime.UtcNow;
                    bool bPassesDateCheck = (CurrentUTCTime >= TestProvision.CreationDate) && (CurrentUTCTime < TestProvision.ExpirationDate);
                    if (!bPassesDateCheck)
                    {
                        Program.LogVerbose("  .. Failed time period check (valid from {0} to {1}, but UTC time is now {2})", TestProvision.CreationDate, TestProvision.ExpirationDate, CurrentUTCTime);
                        continue;
                    }

                    // check to see if we have a certificate for this provision
                    bool bPassesHasMatchingCertCheck = false;
                    if (bCheckCert)
                    {
                        X509Certificate2 Cert = CodeSignatureBuilder.FindCertificate(TestProvision);
                        bPassesHasMatchingCertCheck = (Cert != null);
                        if (bPassesHasMatchingCertCheck && Config.bCert)
                        {
                            bPassesHasMatchingCertCheck &= (CryptoAdapter.GetFriendlyNameFromCert(Cert) == Config.Certificate);
                        }
                    }
                    else
                    {
                        bPassesHasMatchingCertCheck = true;
                    }

                    if (!bPassesHasMatchingCertCheck)
                    {
                        Program.LogVerbose("  .. Failed to find a matching certificate that was in date");
                        continue;
                    }

                    // Made it past all the tests
                    Program.LogVerbose("  Picked '{0}' with AppID '{1}' and Name '{2}' as a matching provision for the game '{3}'", DebugName, TestProvision.ApplicationIdentifier, TestProvision.ProvisionName, GameName);
                    return Key;
                }
            }

            // check to see if there is already an embedded provision
            string EmbeddedMobileProvisionFilename = Path.Combine(Config.RepackageStagingDirectory, "embedded.mobileprovision");

            Program.Warning("Failed to find a valid matching mobile provision, will attempt to use the embedded mobile provision instead if present");
            return EmbeddedMobileProvisionFilename;
        }

        /// <summary>
        /// Extracts the dict values for the Entitlements key and creates a new full .plist file
        /// from them (with outer plist and dict keys as well as doctype, etc...)
        /// </summary>
        public string GetEntitlementsString(string CFBundleIdentifier, out string TeamIdentifier)
        {
            Utilities.PListHelper XCentPList = null;
            Data.ProcessValueForKey("Entitlements", "dict", delegate (XmlNode ValueNode)
            {
                XCentPList = Utilities.PListHelper.CloneDictionaryRootedAt(ValueNode);
            });

            // Modify the application-identifier to be fully qualified if needed
            string CurrentApplicationIdentifier;
            XCentPList.GetString("application-identifier", out CurrentApplicationIdentifier);
            XCentPList.GetString("com.apple.developer.team-identifier", out TeamIdentifier);

            //			if (CurrentApplicationIdentifier.Contains("*"))
            {
                // Replace the application identifier
                string NewApplicationIdentifier = String.Format("{0}.{1}", ApplicationIdentifierPrefix, CFBundleIdentifier);
                XCentPList.SetString("application-identifier", NewApplicationIdentifier);


                // Replace the keychain access groups
                // Note: This isn't robust, it ignores the existing value in the wildcard and uses the same value for
                // each entry.  If there is a legitimate need for more than one entry in the access group list, then
                // don't use a wildcard!
                List<string> KeyGroups = XCentPList.GetArray("keychain-access-groups", "string");

                for (int i = 0; i < KeyGroups.Count; ++i)
                {
                    string Entry = KeyGroups[i];
                    if (Entry.Contains("*"))
                    {
                        Entry = NewApplicationIdentifier;
                    }
                    KeyGroups[i] = Entry;
                }

                XCentPList.SetValueForKey("keychain-access-groups", KeyGroups);
            }

            // must have CloudKit and CloudDocuments for com.apple.developer.icloud-services
            // otherwise the game will not be listed in the Settings->iCloud apps menu on the device
            {
                // iOS only
                if (Platform == "IOS" && XCentPList.HasKey("com.apple.developer.icloud-services"))
                {
                    List<string> ServicesGroups = XCentPList.GetArray("com.apple.developer.icloud-services", "string");
                    ServicesGroups.Clear();

                    ServicesGroups.Add("CloudKit");
                    ServicesGroups.Add("CloudDocuments");
                    XCentPList.SetValueForKey("com.apple.developer.icloud-services", ServicesGroups);
                }

                // For distribution builds, the entitlements from mobileprovisioning have a modified syntax
                if (Config.bForDistribution)
                {
                    // remove the wildcards from the ubiquity-kvstore-identifier string
                    if (XCentPList.HasKey("com.apple.developer.ubiquity-kvstore-identifier"))
                    {
                        string UbiquityKvstoreString;
                        XCentPList.GetString("com.apple.developer.ubiquity-kvstore-identifier", out UbiquityKvstoreString);

                        int DotPosition = UbiquityKvstoreString.LastIndexOf("*");
                        if (DotPosition >= 0)
                        {
                            string TeamPrefix = DotPosition > 1 ? UbiquityKvstoreString.Substring(0, DotPosition - 1) : TeamIdentifier;
                            string NewUbiquityKvstoreIdentifier = String.Format("{0}.{1}", TeamPrefix, CFBundleIdentifier);
                            XCentPList.SetValueForKey("com.apple.developer.ubiquity-kvstore-identifier", NewUbiquityKvstoreIdentifier);
                        }
                    }

                    // remove the wildcards from the ubiquity-container-identifiers array
                    if (XCentPList.HasKey("com.apple.developer.ubiquity-container-identifiers"))
                    {
                        List<string> UbiquityContainerIdentifiersGroups = XCentPList.GetArray("com.apple.developer.ubiquity-container-identifiers", "string");

                        for (int i = 0; i < UbiquityContainerIdentifiersGroups.Count; i++)
                        {
                            int DotPosition = UbiquityContainerIdentifiersGroups[i].LastIndexOf("*");
                            if (DotPosition >= 0)
                            {
                                string TeamPrefix = DotPosition > 1 ? UbiquityContainerIdentifiersGroups[i].Substring(0, DotPosition - 1) : TeamIdentifier;
                                string NewUbiquityContainerIdentifier = String.Format("{0}.{1}", TeamPrefix, CFBundleIdentifier);
                                UbiquityContainerIdentifiersGroups[i] = NewUbiquityContainerIdentifier;
                            }
                        }

                        if (UbiquityContainerIdentifiersGroups.Count == 0)
                        {
                            string NewUbiquityKvstoreIdentifier = String.Format("{0}.{1}", TeamIdentifier, CFBundleIdentifier);
                            UbiquityContainerIdentifiersGroups.Add(NewUbiquityKvstoreIdentifier);
                        }

                        XCentPList.SetValueForKey("com.apple.developer.ubiquity-container-identifiers", UbiquityContainerIdentifiersGroups);
                    }

                    // remove the wildcards from the developer.associated-domains array or string
                    if (XCentPList.HasKey("com.apple.developer.associated-domains"))
                    {
                        string AssociatedDomainsString;
                        XCentPList.GetString("com.apple.developer.associated-domains", out AssociatedDomainsString);

                        //check if the value is string
                        if (AssociatedDomainsString != null && AssociatedDomainsString.Contains("*"))
                        {
                            XCentPList.RemoveKeyValue("com.apple.developer.associated-domains");
                        }
                        else
                        {
                            //check if the value is an array
                            List<string> AssociatedDomainsGroup = XCentPList.GetArray("com.apple.developer.associated-domains", "string");

                            if (AssociatedDomainsGroup.Count == 1 && AssociatedDomainsGroup[0].Contains("*"))
                            {
                                XCentPList.RemoveKeyValue("com.apple.developer.associated-domains");
                            }
                        }
                    }

                    // remove development keys - generated when the cloudkit container is in development mode
                    XCentPList.RemoveKeyValue("com.apple.developer.icloud-container-development-container-identifiers");
                }

                // set the icloud-container-environment according to the project settings
                if (XCentPList.HasKey("com.apple.developer.icloud-container-environment"))
                {
                    List<string> ContainerEnvironmentGroup = XCentPList.GetArray("com.apple.developer.icloud-container-environment", "string");

                    if (ContainerEnvironmentGroup.Count != 0)
                    {
                        ContainerEnvironmentGroup.Clear();

                        // The new value is a string, not an array
                        string NewContainerEnvironment = Config.bForDistribution ? "Production" : "Development";
                        XCentPList.SetValueForKey("com.apple.developer.icloud-container-environment", NewContainerEnvironment);
                    }
                }
            }

            return XCentPList.SaveToString();
        }

        /// <summary>
        /// Constructs a MobileProvision from an xml blob extracted from the real ASN.1 file
        /// </summary>
        public MobileProvision(string EmbeddedPListText)
        {
            Data = new Utilities.PListHelper(EmbeddedPListText);

            // Now extract things

            // Key: ApplicationIdentifierPrefix, Array<String>
            List<string> PrefixList = Data.GetArray("ApplicationIdentifierPrefix", "string");
            if (PrefixList.Count > 1)
            {
                Program.Warning("Found more than one entry for ApplicationIdentifierPrefix in the .mobileprovision, using the first one found");
            }

            if (PrefixList.Count > 0)
            {
                ApplicationIdentifierPrefix = PrefixList[0];
            }

            // Example date string from the XML: "2014-06-30T20:45:55Z";
            DateTimeStyles AppleDateStyle = DateTimeStyles.AllowWhiteSpaces | DateTimeStyles.AssumeUniversal | DateTimeStyles.AdjustToUniversal;

            string CreationDateString;
            if (Data.GetDate("CreationDate", out CreationDateString))
            {
                CreationDate = DateTime.Parse(CreationDateString, CultureInfo.InvariantCulture, AppleDateStyle);
            }

            string ExpirationDateString;
            if (Data.GetDate("ExpirationDate", out ExpirationDateString))
            {
                ExpirationDate = DateTime.Parse(ExpirationDateString, CultureInfo.InvariantCulture, AppleDateStyle);
            }

            // Key: DeveloperCertificates, Array<Data> (uuencoded)
            string CertificatePassword = "";
            List<string> CertificateList = Data.GetArray("DeveloperCertificates", "data");
            foreach (string EncodedCert in CertificateList)
            {
                byte[] RawCert = Convert.FromBase64String(EncodedCert);
                DeveloperCertificates.Add(new X509Certificate2(RawCert, CertificatePassword));
            }

            // Key: Name, String
            if (!Data.GetString("Name", out ProvisionName))
            {
                ProvisionName = "(unknown)";
            }

            // Key: ProvisionedDevices, Array<String>
            ProvisionedDeviceIDs = Data.GetArray("ProvisionedDevices", "string");

            // Key: application-identifier, Array<String>
            Utilities.PListHelper XCentPList = null;
            Data.ProcessValueForKey("Entitlements", "dict", delegate (XmlNode ValueNode)
            {
                XCentPList = Utilities.PListHelper.CloneDictionaryRootedAt(ValueNode);
            });

            // Modify the application-identifier to be fully qualified if needed
            if (!XCentPList.GetString("application-identifier", out ApplicationIdentifier))
            {
                ApplicationIdentifier = "(unknown)";
            }

            // check for get-task-allow
            bDebug = XCentPList.GetBool("get-task-allow");

            if (!Data.GetString("UUID", out UUID))
            {
                UUID = "(unkown)";
            }

            List<string> Platforms = Data.GetArray("Platform", "string");
            if (Platforms.Contains("iOS"))
            {
                Platform = "IOS";
            }
            else if (Platforms.Contains("tvOS"))
            {
                Platform = "TVOS";
            }
            else
            {
                Platform = "";
            }
        }

        /// <summary>
        /// Does this provision contain the specified UDID?
        /// </summary>
        public bool ContainsUDID(string UDID)
        {
            bool bFound = false;
            foreach (string TestUDID in ProvisionedDeviceIDs)
            {
                if (TestUDID.Equals(UDID, StringComparison.InvariantCultureIgnoreCase))
                {
                    bFound = true;
                    break;
                }
            }

            return bFound;
        }
    }

    /// <summary>
    /// This class understands how to get the embedded plist in a .mobileprovision file.  It doesn't
    /// understand the full format and is not capable of writing a new one out or anything similar.
    /// </summary>
    public class MobileProvisionParser
    {
        private static int StrStrByteArray(byte[] Haystack, int Offset, string Needle)
        {
            byte[] NeedleBytes = Encoding.UTF8.GetBytes(Needle);

            //@TODO: Is there anything better in .NET? That's going to be pretty slow on large files
            for (int i = Offset; i < Haystack.Length - NeedleBytes.Length; ++i)
            {
                bool bMatch = true;

                for (int j = 0; j < NeedleBytes.Length; ++j)
                {
                    if (Haystack[i + j] != NeedleBytes[j])
                    {
                        bMatch = false;
                        break;
                    }
                }

                if (bMatch)
                {
                    return i;
                }
            }

            return -1;
        }

        public static MobileProvision ParseFile(byte[] RawData)
        {
            //@TODO: This file is just an ASN.1 stream, should find or make a raw ASN1 parser and use
            // that instead of this (theoretically fragile) code (particularly the length extraction)

            string StartPattern = "<?xml";
            string EndPattern = "</plist>";

            // Search the start pattern
            int StartPos = StrStrByteArray(RawData, 0, StartPattern);
            if (StartPos != -1)
            {
                // Search the end pattern
                int EndPos = StrStrByteArray(RawData, StartPos, EndPattern);
                if (EndPos != -1)
                {
                    // Offset the end position to take in account the end pattern
                    EndPos += EndPattern.Length;

                    // Convert the data to a string
                    string PlistText = Encoding.UTF8.GetString(RawData, StartPos, EndPos - StartPos);

                    // Return the constructed 'mobile provision'
                    return new MobileProvision(PlistText);
                }
            }

            // Unable to find the start of the plist data
            Program.Error("Failed to find embedded plist in .mobileprovision file");
            return null;
        }

        public static MobileProvision ParseFile(Stream InputStream)
        {
            // Read in the entire file
            int NumBytes = (int)InputStream.Length;
            byte[] RawData = new byte[NumBytes];
            InputStream.Read(RawData, 0, NumBytes);

            return ParseFile(RawData);
        }


        public static MobileProvision ParseFile(string Filename)
        {
            FileStream InputStream = File.OpenRead(Filename);
            MobileProvision Result = ParseFile(InputStream);
            InputStream.Close();
            Result.FileName = Filename;

            return Result;
        }

        /// <summary>
        /// Opens embedded.mobileprovision from within an IPA
        /// </summary>
        public static MobileProvision ParseIPA(string Filename)
        {
            FileOperations.ReadOnlyZipFileSystem FileSystem = new FileOperations.ReadOnlyZipFileSystem(Filename);
            MobileProvision Result = ParseFile(FileSystem.ReadAllBytes("embedded.mobileprovision"));
            FileSystem.Close();

            return Result;
        }
    }
}