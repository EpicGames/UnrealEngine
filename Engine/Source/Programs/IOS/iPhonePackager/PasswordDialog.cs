// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Drawing;
using System.Linq;
using System.Text;
using System.Windows.Forms;

namespace iPhonePackager
{
	public partial class PasswordDialog : Form
	{
		string PasswordResult = null;

		public PasswordDialog()
		{
			InitializeComponent();
		}

		public static bool RequestPassword(out string Password)
		{
			PasswordDialog Dlg = new PasswordDialog();
			Dlg.ShowDialog();

			Password = Dlg.PasswordResult;
			return Password != null;
		}

		private void OKButton_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.OK;
			PasswordResult = PasswordEntryBox.Text;
			Close();
		}
	}
}
