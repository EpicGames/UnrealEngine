<?xml version="1.0" encoding="utf-8" ?>
<configuration>
	<runtime>
		<generatePublisherEvidence enabled="false"/>
    <loadFromRemoteSources enabled="true"/>
	</runtime>
</configuration>
