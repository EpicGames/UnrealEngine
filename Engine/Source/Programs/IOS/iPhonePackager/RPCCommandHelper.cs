// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Net.NetworkInformation;
using System.Net.Sockets;
using System.Text;
using System.Threading;
using System.Threading.Tasks;

namespace DotNETUtilities
{
    public class RPCCommandHelper
    {
        // uploads/downloads are broken up into smaller chunks no bigger than this
        const int MaxChunkSize = 10 * 1024 * 1024;

        /**
		 * Utility for simply pinging a remote host to see if it's alive.
		 * Only returns true if the host is pingable, false for all
		 * other cases.
		 */
        public static bool PingRemoteHost(string RemoteHostNameOrAddressToPing)
        {
            bool PingSuccess = false;
            try
            {
                // The default Send method will timeout after 5 seconds
                Ping PingSender = new Ping();
                PingReply Reply = PingSender.Send(RemoteHostNameOrAddressToPing);
                if (Reply.Status == IPStatus.Success)
                {
                    Debug.WriteLine("Successfully pinged " + RemoteHostNameOrAddressToPing);
                    PingSuccess = true;
                }
            }
            catch (Exception Ex)
            {
                // This will catch any error conditions like unknown hosts
                Console.WriteLine("Failed to ping " + RemoteHostNameOrAddressToPing);
                Console.WriteLine("Exception details: " + Ex.ToString());
            }
            return PingSuccess;
        }

        /**
		 * Utility for making a socket and connecting to UnrealRemoteTool
		 * The CALLER MUST HANDLE THE EXCEPTIONS!
		 */
        public static Socket ConnectToUnrealRemoteTool(string MachineName)
        {
            Socket S = new Socket(AddressFamily.InterNetwork, SocketType.Stream, ProtocolType.IP);
            S.Connect(MachineName, 8199);
            return S;
        }

        const Int64 TypeString = 0;
        const Int64 TypeLong = 1;
        const Int64 TypeBlob = 2;

        private static UTF8Encoding Encoder = new UTF8Encoding();

        private static byte[] TypeStringBytes = BitConverter.GetBytes(TypeString);
        private static byte[] TypeLongBytes = BitConverter.GetBytes(TypeLong);
        private static byte[] TypeBlobBytes = BitConverter.GetBytes(TypeBlob);

        private static void SendString(MemoryStream Stream, string Value)
        {
            byte[] StringBytes = Encoder.GetBytes(Value);

            // write a string with type and info as one unit
            Stream.Write(TypeStringBytes, 0, 8);
            Stream.Write(BitConverter.GetBytes((Int64)Value.Length), 0, 8);
            Stream.Write(StringBytes, 0, StringBytes.Length);
        }

        private static void SendLong(MemoryStream Stream, Int64 Value)
        {
            // write a long with type and info as one unit
            Stream.Write(TypeLongBytes, 0, 8);
            Stream.Write(BitConverter.GetBytes(Value), 0, 8);
        }

        private static void SendBlob(MemoryStream Stream, byte[] Value)
        {
            // write a blob with type and info as one unit
            Stream.Write(TypeBlobBytes, 0, 8);
            Stream.Write(BitConverter.GetBytes((Int64)Value.Length), 0, 8);
            Stream.Write(Value, 0, Value.Length);
        }

        private static object ReadObject(byte[] MessageBuffer, ref Int64 Offset)
        {
            // first, read the type
            Int64 Type = BitConverter.ToInt64(MessageBuffer, (int)Offset);
            Offset += 8;

            if (Type == TypeLong)
            {
                // make the object
                Int64 Value = BitConverter.ToInt64(MessageBuffer, (int)Offset);
                Offset += 8;
                return Value;
            }
            else if (Type == TypeString)
            {
                Int64 Length = BitConverter.ToInt64(MessageBuffer, (int)Offset);
                Offset += 8;

                // pull the string out of the buffer
                string Value = Encoder.GetString(MessageBuffer, (int)Offset, (int)Length);
                Offset += Length;
                return Value;
            }
            else if (Type == TypeBlob)
            {
                Int64 Length = BitConverter.ToInt64(MessageBuffer, (int)Offset);
                Offset += 8;

                // pull the blob out of the buffer
                byte[] Value = new byte[Length];
                Buffer.BlockCopy(MessageBuffer, (int)Offset, Value, 0, (int)Length);
                Offset += Length;

                return Value;
            }
            else
            {
                throw new Exception("Received invalid type!");
            }
        }

        private static Hashtable SendMessageWithReply(Socket Socket, Hashtable Message)
        {
            // this holds an entire message
            MemoryStream Stream = new MemoryStream();

            // leave space for the message length in the message (we will fill this again)
            Int64 MessageLength = 0;
            Stream.Write(BitConverter.GetBytes(MessageLength), 0, 8);

            Int64 NumEntries = Message.Count;
            SendLong(Stream, NumEntries);

            // send each entry in the message
            foreach (string Key in Message.Keys)
            {
                SendString(Stream, Key);

                object Value = Message[Key];
                Type ValueType = Value.GetType();
                if (ValueType.Equals(typeof(string)))
                {
                    SendString(Stream, Value as string);
                }
                else if (ValueType.Equals(typeof(byte[])))
                {
                    SendBlob(Stream, Value as byte[]);
                }
                else if (ValueType.Equals(typeof(Int32)))
                {
                    SendLong(Stream, (Int64)(Int32)Value);
                }
                else if (ValueType.Equals(typeof(Int64)))
                {
                    SendLong(Stream, (Int64)Value);
                }
                else if (ValueType.Equals(typeof(UInt32)))
                {
                    SendLong(Stream, (Int64)(UInt32)Value);
                }
                else if (ValueType.Equals(typeof(UInt64)))
                {
                    SendLong(Stream, (Int64)(UInt64)Value);
                }
            }

            // go back to the start, and write the length
            Stream.Seek(0, SeekOrigin.Begin);
            MessageLength = Stream.Length;
            Stream.Write(BitConverter.GetBytes(MessageLength), 0, 8);

            // send the message!
            Socket.Send(Stream.ToArray());

            try
            {
                // first, read the type
                byte[] LengthBuffer = new byte[8];
                if (Socket.Receive(LengthBuffer) != LengthBuffer.Length)
                {
                    throw new Exception("Failed to receive the message length, perhaps the server has died...");
                }

                MessageLength = BitConverter.ToInt64(LengthBuffer, 0);

                byte[] MessageBuffer = new byte[MessageLength];
                // length includes the bytes for the length
                Int64 AmountToRead = MessageLength - sizeof(Int64);
                Int64 ReadOffset = 0;
                while (AmountToRead > 0)
                {
                    // pull some data from the network
                    int AmountRead = Socket.Receive(MessageBuffer, (int)ReadOffset, (int)AmountToRead, SocketFlags.None);
                    AmountToRead -= AmountRead;
                    ReadOffset += AmountRead;

                    // if none came, we are dead
                    if (AmountRead == 0)
                    {
                        throw new Exception("Failed to receive the full message, perhaps the server has died...");
                    }
                }

                // now parse the reply
                Int64 Offset = 0;
                NumEntries = (Int64)ReadObject(MessageBuffer, ref Offset);
                Hashtable Reply = new Hashtable();

                for (int Index = 0; Index < NumEntries; Index++)
                {
                    // read the key
                    string Key = (string)ReadObject(MessageBuffer, ref Offset);
                    object Value = ReadObject(MessageBuffer, ref Offset);

                    // add it to the reply hashtable
                    Reply[Key] = Value;
                }

                return Reply;
            }
            catch (Exception Ex)
            {
                Console.WriteLine(Ex.Message);
                return null;
            }
        }
        public static void DummyFileCommand(Socket Socket)
        {
            Hashtable CommandParameters = new Hashtable();
            CommandParameters["CommandName"] = "rpc:dummy";

            // Try to execute the command
            SendMessageWithReply(Socket, CommandParameters);
        }

        public static void MakeDirectory(Socket Socket, string DirName)
        {
            Hashtable CommandParameters = new Hashtable();
            CommandParameters["CommandName"] = "rpc:makedirectory";
            CommandParameters["CommandArgs"] = DirName;

            // Try to execute the command
            SendMessageWithReply(Socket, CommandParameters);
        }

        public static int GetCommandSlots(Socket Socket)
        {
            Hashtable CommandParameters = new Hashtable();
            CommandParameters["CommandName"] = "rpc:command_slots_available";

            Hashtable Result = SendMessageWithReply(Socket, CommandParameters);

            int CommandSlots = Convert.ToInt32(Result["CommandOutput"]);

            return CommandSlots;
        }

        public static void RemoveDirectory(Socket Socket, string DirName)
        {
            Hashtable CommandParameters = new Hashtable();
            CommandParameters["CommandName"] = "rpc:removedirectory";
            CommandParameters["CommandArgs"] = DirName;

            // Try to execute the command
            SendMessageWithReply(Socket, CommandParameters);
        }

        public static void RPCBatchUpload(Socket Socket, string[] CopyCommands)
        {
            // make a new structure to send over the wire with local filetime, and remote filename
            List<string> LocalNames = new List<string>();
            List<string> RemoteNames = new List<string>();

            // allocate a buffer of Int64s to send over the wire
            byte[] LocalTimes = new byte[sizeof(Int64) * CopyCommands.Length];
            StringBuilder RemoteNameString = new StringBuilder();

            int FileIndex = 0;
            foreach (string Pair in CopyCommands)
            {
                // each pair is a local and remote filename
                string[] Filenames = Pair.Split(";".ToCharArray());
                if (Filenames.Length == 2)
                {
                    FileInfo Info = new FileInfo(Filenames[0]);
                    if (Info.Exists)
                    {
                        // make the remote filename -> local filetime pair
                        LocalNames.Add(Filenames[0]);
                        RemoteNames.Add(Filenames[1]);

                        // store remote name
                        RemoteNameString.Append(Filenames[1] + "\n");

                        // store local time
                        Int64 LocalTime = ToRemoteTime(Info.LastWriteTimeUtc);
                        byte[] LocalTimeBytes = BitConverter.GetBytes(LocalTime);
                        LocalTimeBytes.CopyTo(LocalTimes, sizeof(Int64) * FileIndex);

                        // move to next file
                        FileIndex++;
                    }
                    else
                    {
                        Console.WriteLine("Tried to batch upload non-existant file: " + Filenames[0]);
                    }
                }
            }

            Hashtable CommandParameters = new Hashtable();
            CommandParameters["CommandName"] = "rpc:getnewerfiles";
            CommandParameters["DestNames"] = RemoteNameString.ToString();
            CommandParameters["SourceTimes"] = LocalTimes;

            Hashtable Results = SendMessageWithReply(Socket, CommandParameters);

            // ask the remote server for the files that are newer
            byte[] NewerFileIndices = Results["NewerFiles"] as byte[];
            if (NewerFileIndices != null)
            {
                int NumFiles = NewerFileIndices.Length / sizeof(Int64);
                for (int Index = 0; Index < NumFiles; Index++)
                {
                    FileIndex = (int)BitConverter.ToInt64(NewerFileIndices, Index * sizeof(Int64));

                    // now send it over
                    Hashtable Result = RPCUpload(Socket, LocalNames[FileIndex], RemoteNames[FileIndex]);

                    if (Result == null)
                    {
                        // we want to fail here so builds don't quietly fail, but we use a useful message
                        throw new Exception(string.Format("Failed to upload local file {0} to {1}", LocalNames[FileIndex], RemoteNames[FileIndex]));
                    }
                    else
                    {
                        Console.WriteLine(Result["CommandOutput"] as string);
                    }
                }
            }
        }

        /**
		 * FileList is a \n separated list of files
		 */
        public static Int64[] RPCBatchFileInfo(Socket Socket, string FileList)
        {
            Hashtable CommandParameters = new Hashtable();
            CommandParameters["CommandName"] = "rpc:batchfileinfo";
            CommandParameters["Files"] = FileList;
            CommandParameters["ClientNow"] = ToRemoteTime(DateTime.UtcNow);

            Hashtable Result = SendMessageWithReply(Socket, CommandParameters);

            // get the results
            byte[] RawData = Result["FileInfo"] as byte[];

            // put in terms of Int64s
            int NumInt64s = RawData.Length / sizeof(Int64);
            Int64[] Ints = new Int64[NumInt64s];

            // now copy the Int64s out into the nice array
            for (int Index = 0; Index < NumInt64s; Index++)
            {
                Int64 Value = BitConverter.ToInt64(RawData, Index * sizeof(Int64));
                Ints[Index] = Value;
            }

            return Ints;
        }

        public static Hashtable RPCUpload(Socket Socket, string SourceFilename, string DestFilename)
        {
            // read in the source file
            try
            {
                bool bDone = false;
                Hashtable FinalResult = null;
                Int64 Offset = 0;
                FileStream Stream = File.Open(SourceFilename, FileMode.Open, FileAccess.Read, FileShare.Read);
                long FileLength = Stream.Length;
                while (!bDone)
                {
                    int AmountToRead = Math.Min((int)(Stream.Length - Offset), RPCCommandHelper.MaxChunkSize);

                    byte[] Contents = new byte[AmountToRead];
                    Stream.Read(Contents, 0, AmountToRead);

                    Hashtable CommandParameters = new Hashtable();
                    CommandParameters["CommandName"] = "rpc:upload";
                    CommandParameters["Contents"] = Contents;
                    CommandParameters["CommandArgs"] = DestFilename;
                    CommandParameters["bAppend"] = (Int64)((Offset > 0) ? 1 : 0); // we append when reading past the first one
                    CommandParameters["SrcFiletime"] = ToRemoteTime(File.GetLastWriteTimeUtc(SourceFilename));

                    // send it over the remoting network
                    FinalResult = SendMessageWithReply(Socket, CommandParameters);

                    Offset += AmountToRead;
                    if (Offset == FileLength)
                    {
                        bDone = true;
                    }
                }
                Stream.Close();

                return FinalResult;
            }
            catch (System.Exception Ex)
            {
                Console.WriteLine("Failed reading file " + Ex.ToString());
                return null;
            }
        }

        public static Hashtable RPCDownload(Socket Socket, string SourceFilename, string DestFilename)
        {
            Hashtable CommandParameters = new Hashtable();
            CommandParameters["CommandName"] = "rpc:download";
            CommandParameters["CommandArgs"] = SourceFilename;
            CommandParameters["MaxChunkSize"] = (Int64)MaxChunkSize;
            CommandParameters["Start"] = (Int64)0;

            // read it over the remoting network (contents will be in the result)
            Hashtable CommandResult = SendMessageWithReply(Socket, CommandParameters);

            // write out the bytes we got over the network
            byte[] Contents = (byte[])CommandResult["Contents"];
            if (Contents != null)
            {
                Directory.CreateDirectory(Path.GetDirectoryName(DestFilename));
                File.WriteAllBytes(DestFilename, Contents);
            }

            int AmountRead = Contents.Length;
            while (CommandResult["NeedsToContinue"] != null)
            {
                CommandParameters["Start"] = (Int64)AmountRead;
                CommandResult = SendMessageWithReply(Socket, CommandParameters);

                // read more chunks
                Contents = (byte[])CommandResult["Contents"];
                if (Contents != null)
                {
                    FileStream Stream = File.Open(DestFilename, FileMode.Append);
                    Stream.Write(Contents, 0, Contents.Length);
                    AmountRead += Contents.Length;
                    Stream.Close();
                }
            }

            return CommandResult;
        }

        public static DateTime FromRemoteTime(Int64 RemoteTime)
        {
            // remote time is seconds since 2001, we need ticks since 1 (it could have been negative, so just clamp to 0 in this case)
            Int64 SecondsSince2001 = Math.Max(0, RemoteTime);
            Int64 TicksSince2001 = SecondsSince2001 * 10000 * 1000;

            // make a DateTime object
            DateTime ConvertedTime = new DateTime(TicksSince2001, DateTimeKind.Utc);

            // windows time is since the year 1, so add 2000 years to put into year 1 space
            ConvertedTime = ConvertedTime.AddYears(2000);

            return ConvertedTime;
        }

        public static Int64 ToRemoteTime(DateTime LocalTime)
        {
            // if we are before 2001, just assume start of 2001
            if (LocalTime.Year < 2001)
            {
                return 0;
            }

            // put time into 2001 space, so subtract off 2000 years
            DateTime ConvertedTime = LocalTime.AddYears(-2000);

            Int64 TicksSince2001 = ConvertedTime.Ticks;
            // ticks are 10000 per millisecond
            Int64 SecondsSince2001 = TicksSince2001 / (10000 * 1000);

            return SecondsSince2001;
        }

        public static bool GetFileInfo(Socket Socket, string Filename, DateTime ClientNow, out DateTime ModificationTime, out long Length)
        {
            Hashtable CommandParameters = new Hashtable();
            CommandParameters["CommandName"] = "rpc:getfileinfo";
            CommandParameters["CommandArgs"] = Filename;
            CommandParameters["ClientNow"] = ToRemoteTime(ClientNow);

            Hashtable Results = SendMessageWithReply(Socket, CommandParameters);
            // convert time into local time space
            ModificationTime = FromRemoteTime((Int64)Results["ModificationTime"]);
            Length = (Int64)Results["Length"];

            // Length will be -1 if the file didn't exist
            return Length != -1;
        }

        public static Hashtable RPCCommand(Socket Socket, string WorkingDirectory, string CommandName, string CommandArgs, string RemoteOutputPath)
        {
            // set up parameters
            Hashtable CommandParameters = new Hashtable();
            CommandParameters["WorkingDirectory"] = WorkingDirectory;
            CommandParameters["CommandName"] = CommandName;
            CommandParameters["CommandArgs"] = CommandArgs;
            if (RemoteOutputPath != null)
            {
                CommandParameters["OutputFile"] = RemoteOutputPath;
            }

            // Try to execute the command
            return SendMessageWithReply(Socket, CommandParameters);
        }


        public static void RPCSoakTest(Socket Socket)
        {
            for (int Index = 0; Index < 24; Index++)
            {
                string Filename = "/UnrealEngine3/soak" + Index.ToString() + ".bin";
                Thread SoakThread = new Thread(delegate ()
                {
                    while (true)
                    {
                        RPCUpload(Socket, "C:\\soak.bin", Filename);
                        DateTime M;
                        long L;
                        GetFileInfo(Socket, Filename, DateTime.UtcNow, out M, out L);
                    }
                });
                SoakThread.Start();
            }
            while (true)
            {
                Thread.Sleep(1000);
            }
        }

    }
}
