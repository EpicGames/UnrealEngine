/**
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Text;

namespace iPhonePackager
{
	public partial class SSHCommandHelper
	{
		private static string SSHExePath = "";
		private static string SSHAuthentication = "";
		private static string SSHUser = "";
		private static string RSyncExePath = "";

		/** 
		 * Parse the commandline for SSH arguments
		 */
		public static bool ParseSSHProperties(string[] Arguments)
		{
			Int32 ArgIndex = 0;
			foreach( string NextArg in Arguments)
			{
				string Arg = NextArg.ToLowerInvariant();

				if (Arg.StartsWith("-"))
				{
					if (Arg == "-usessh")
					{
						Config.bUseRPCUtil = false;
					}
					else if (Arg == "-sshexe")
					{
						if (Arguments.Length > ArgIndex + 1)
						{
							SSHExePath = Arguments[ArgIndex+1];
						}
						else
						{
							return false;
						}
					}
					else if (Arg == "-sshauth")
					{
						if (Arguments.Length > ArgIndex + 1)
						{
							SSHAuthentication = Arguments[ArgIndex + 1];
						}
						else
						{
							return false;
						}
					}
					else if (Arg == "-sshuser")
					{
						if (Arguments.Length > ArgIndex + 1)
						{
							SSHUser = Arguments[ArgIndex + 1];
						}
						else
						{
							return false;
						}
					}
					else if (Arg == "-rsyncexe")
					{
						if (Arguments.Length > ArgIndex + 1)
						{
							RSyncExePath = Arguments[ArgIndex + 1];
						}
						else
						{
							return false;
						}
					}
					else if (Arg == "-overridedevroot")
					{
						if (Arguments.Length > ArgIndex + 1)
						{
							Config.OverrideDevRoot = Arguments[ArgIndex + 1];
						}
						else
						{
							return false;
						}
					}
				}
				++ArgIndex;
			}

			bool bValidSetup = true;
			if (Config.bUseRPCUtil == false)
			{
				// If we aren't using rpc utility, 
				// ensure we have all the arguments needed for our ssh and rsync commands to run
				bValidSetup = ValidateSetup();
			}

			return bValidSetup;
		}

		private static bool ValidateSetup()
		{
			bool bIsSetupForSSH = true;

			if( string.IsNullOrEmpty(SSHExePath) )
			{
				Console.WriteLine("SSH requires the -sshexe commandline parameter");
				bIsSetupForSSH = false;
			}
			else if (string.IsNullOrEmpty(SSHAuthentication))
			{
				Console.WriteLine("SSH requires the -sshauth commandline parameter");
				bIsSetupForSSH = false;
			}
			else if (string.IsNullOrEmpty(SSHUser))
			{
				Console.WriteLine("SSH requires the -sshuser commandline parameter");
				bIsSetupForSSH = false;
			}
			else if (string.IsNullOrEmpty(RSyncExePath))
			{
				Console.WriteLine("SSH requires the -rsyncexe commandline parameter");
				bIsSetupForSSH = false;
			}
			else if (string.IsNullOrEmpty(Config.OverrideDevRoot))
			{
				Console.WriteLine("For now, SSH requires the -overridedevroot commandline parameter");
				bIsSetupForSSH = false;
			}

			return bIsSetupForSSH;
		}


		// SSH command output, tracks output and errors from the remote execution
		private static Dictionary<Object, StringBuilder> SSHOutputMap = new Dictionary<object, StringBuilder>();
        public static Hashtable SSHReturn = new Hashtable();
		/** 
		 * Execute a console command on the remote mac.
		 */
		static public bool Command(string RemoteMac, string Command, string WorkingFolder)
		{
			// make the commandline for other end
			string RemoteCommandline = "cd " + WorkingFolder + " && " + Command;

			Process SSHProcess = new Process();

			// Get the executable dir for SSH
			string ExeDir = Path.GetDirectoryName(SSHExePath);
			if (ExeDir != "")
			{
				SSHProcess.StartInfo.WorkingDirectory = ExeDir;
			}

			SSHProcess.StartInfo.FileName = SSHExePath;
			SSHProcess.StartInfo.Arguments = string.Format(
				"{0} \"{1}@{2}\" \"{3}\"",
				SSHAuthentication,
				SSHUser,
				RemoteMac,
				RemoteCommandline.Replace("\"", "\\\""));

			// add this process to the map
			SSHOutputMap[SSHProcess] = new StringBuilder("");

			SSHProcess.StartInfo.UseShellExecute = false;
			SSHProcess.StartInfo.RedirectStandardOutput = true;
			SSHProcess.OutputDataReceived += new DataReceivedEventHandler(OutputReceivedForSSH);
			SSHProcess.StartInfo.RedirectStandardError = true;
			SSHProcess.ErrorDataReceived += new DataReceivedEventHandler(OutputReceivedForSSH);

			DateTime Start = DateTime.Now;
			SSHProcess.Start();

			SSHProcess.BeginOutputReadLine();
			SSHProcess.BeginErrorReadLine();

			SSHProcess.WaitForExit();

			Console.WriteLine("Execute took {0}", (DateTime.Now - Start).ToString());

			// now we have enough to fill out the HashTable
			SSHReturn = new Hashtable();
            SSHReturn["CommandOutput"] = SSHOutputMap[SSHProcess].ToString();
            SSHReturn["ExitCode"] = (object)SSHProcess.ExitCode;

			SSHOutputMap.Remove(SSHProcess);

			return SSHProcess.ExitCode == 0;
		}

		/** 
		 * Upload a number of files to the remote mac
		 */
		public static void BatchUpload(string RemoteMac, string[] CopyCommands)
		{
			// make a new structure to send over the wire with local filetime, and remote filename
			List<string> LocalNames = new List<string>();
			List<string> RemoteNames = new List<string>();

			foreach (string Pair in CopyCommands)
			{
				// each pair is a local and remote filename
				string[] Filenames = Pair.Split(";".ToCharArray());

				if (Filenames.Length == 2)
				{
					FileInfo Info = new FileInfo(Filenames[0]);
					if (Info.Exists)
					{
						// make the remote filename -> local filetime pair
						LocalNames.Add(Filenames[0]);
						RemoteNames.Add(Filenames[1]);
					}
					else
					{
						Console.WriteLine("Tried to batch upload non-existant file: " + Filenames[0]);
					}
				}
			}

			
			for (int Index = 0; Index < LocalNames.Count; Index++)
			{
				// now send it over
				Hashtable Result = UploadFile(RemoteMac, LocalNames[Index], RemoteNames[Index]);

				if (Result == null)
				{
					// we want to fail here so builds don't quietly fail, but we use a useful message
					throw new Exception(string.Format("Failed to upload local file {0} to {1}", LocalNames[Index], RemoteNames[Index]));
				}
			}
		}

		/** 
		 * Upload a single file to the remote mac
		 */
		static public Hashtable UploadFile(string RemoteMac, string LocalPath, string RemotePath)
		{
			string RemoteDir = Path.GetDirectoryName(RemotePath).Replace("\\", "/");
			RemoteDir = RemoteDir.Replace(" ", "\\ ");
			string RemoteFilename = Path.GetFileName(RemotePath);

			// get the executable dir for SSH, so Rsync can call it easily
			string ExeDir = Path.GetDirectoryName(SSHExePath);

			Process RsyncProcess = new Process();
			if (ExeDir != "")
			{
				RsyncProcess.StartInfo.WorkingDirectory = ExeDir;
			}

			// make simple rsync commandline to send a file
			RsyncProcess.StartInfo.FileName = RSyncExePath;
			RsyncProcess.StartInfo.Arguments = string.Format(
                "-zrltgoDe \"./ssh {0}\" --rsync-path=\"mkdir -p {1} && rsync\" --chmod=ug=rwX,o=rxX '{2}' \"{3}@{4}\":'{1}/{5}'",
				SSHAuthentication,
				RemoteDir,
				ConvertPathToCygwin(LocalPath),
				SSHUser,
				RemoteMac,
				RemoteFilename
				);

			SSHOutputMap[RsyncProcess] = new StringBuilder("");

			RsyncProcess.StartInfo.UseShellExecute = false;
			RsyncProcess.StartInfo.RedirectStandardOutput = true;
			RsyncProcess.OutputDataReceived += new DataReceivedEventHandler(OutputReceivedForRsync);
			RsyncProcess.StartInfo.RedirectStandardError = true;
			RsyncProcess.ErrorDataReceived += new DataReceivedEventHandler(OutputReceivedForRsync);

			RsyncProcess.Start();

			RsyncProcess.BeginOutputReadLine();
			RsyncProcess.BeginErrorReadLine();

			RsyncProcess.WaitForExit();

			// now we have enough to fill out the HashTable
			Hashtable ReturnHash = new Hashtable();
			ReturnHash["CommandOutput"] = SSHOutputMap[RsyncProcess].ToString();
			ReturnHash["ExitCode"] = (object)RsyncProcess.ExitCode;

			// run rsync (0 means success)
			return ReturnHash;
		}


		/** 
		 * Download a file from the remote mac
		 */
		static public Hashtable DownloadFile(string RemoteMac, string RemotePath, string LocalPath)
		{
			// get the executable dir for SSH, so Rsync can call it easily
			string ExeDir = Path.GetDirectoryName(SSHExePath);
			string RemoteDir = RemotePath.Replace("\\", "/");
			RemoteDir = RemoteDir.Replace(" ", "\\ ");

			Process RsyncProcess = new Process();
			if (ExeDir != "")
			{
				RsyncProcess.StartInfo.WorkingDirectory = ExeDir;
			}

			// make sure directory exists to download to
			Directory.CreateDirectory(Path.GetDirectoryName(LocalPath));

			// make simple rsync commandline to send a file
			RsyncProcess.StartInfo.FileName = RSyncExePath;
			RsyncProcess.StartInfo.Arguments = string.Format(
                "-zrltgoDe \"./ssh {0}\" \"{2}@{3}\":'{4}' \"{1}\"",
				SSHAuthentication,
				ConvertPathToCygwin(LocalPath),
				SSHUser,
				RemoteMac,
				RemoteDir
				);

			SSHOutputMap[RsyncProcess] = new StringBuilder("");

			RsyncProcess.StartInfo.UseShellExecute = false;
			RsyncProcess.StartInfo.RedirectStandardOutput = true;
			RsyncProcess.OutputDataReceived += new DataReceivedEventHandler(OutputReceivedForRsync);
			RsyncProcess.StartInfo.RedirectStandardError = true;
			RsyncProcess.ErrorDataReceived += new DataReceivedEventHandler(OutputReceivedForRsync);

			RsyncProcess.Start();

			RsyncProcess.BeginOutputReadLine();
			RsyncProcess.BeginErrorReadLine();

			RsyncProcess.WaitForExit();

			// now we have enough to fill out the HashTable
			Hashtable ReturnHash = new Hashtable();
			ReturnHash["CommandOutput"] = SSHOutputMap[RsyncProcess].ToString();
			ReturnHash["ExitCode"] = (Int64)RsyncProcess.ExitCode;

			// run rsync (0 means success)
			return ReturnHash;
		}


		static public void OutputReceivedForRsync(Object Sender, DataReceivedEventArgs Line)
		{
			if ((Line != null) && (Line.Data != null) && (Line.Data != ""))
			{
				Program.Log(Line.Data);
			}
		}


		static public void OutputReceivedForSSH(Object Sender, DataReceivedEventArgs Line)
		{
			if ((Line != null) && (Line.Data != null) && (Line.Data != ""))
			{
                Program.Log(Line.Data);
				StringBuilder SSHOutput = SSHOutputMap[Sender];
				if (SSHOutput.Length != 0)
				{
					SSHOutput.Append(Environment.NewLine);
				}
				SSHOutput.Append(Line.Data);
			}
		}


		private static string ConvertPathToCygwin(string InPath)
		{
			if (InPath == null)
			{
				return null;
			}
			return "/cygdrive/" + InPath.Replace("\\\\", "/").Replace("\\", "/").Replace(":", "");
		}
	}
}