// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Drawing;
using System.Text;
using System.Windows.Forms;

namespace iPhonePackager
{
	public partial class SlowProgressDialog : Form
	{
		public SlowProgressDialog()
		{
			InitializeComponent();
		}

		public delegate void WorkToDoSignature(BackgroundWorker BGWorker);
		public WorkToDoSignature OnBeginBackgroundWork = null;

		private void SlowWork_RunWorkerCompleted(object sender, RunWorkerCompletedEventArgs e)
		{
			DialogResult = DialogResult.OK;
			ProgressIndicator.Value = 0;
			StatusLabel.Text = "";
		}

		private void SlowWork_DoWork(object sender, DoWorkEventArgs e)
		{
			Program.BGWorker = SlowWork;
			Program.ProgressIndex = 0;
			try
			{
				OnBeginBackgroundWork(SlowWork);
			}
			catch (Exception ex)
			{
				Program.Error(ex.Message);
			}
			Program.BGWorker = null;
		}

		private void SlowWork_ProgressChanged(object sender, ProgressChangedEventArgs e)
		{
			ProgressIndicator.Value = e.ProgressPercentage;
			StatusLabel.Text = e.UserState as string;
		}

		private void SlowProgressDialog_Shown(object sender, EventArgs e)
		{
			if (OnBeginBackgroundWork != null)
			{
				SlowWork.RunWorkerAsync();
			}
			else
			{
				DialogResult = DialogResult.Abort;
			}
		}
	}
}
