// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Drawing;
using System.Linq;
using System.Text;
#if !IPP_CLI
using System.Windows.Forms;
#endif
using System.IO;
using System.Security.Cryptography.X509Certificates;
using System.Diagnostics;
using System.Runtime.InteropServices;

namespace iPhonePackager
{
	public partial class ToolsHub
#if !IPP_CLI
		: Form
#endif
	{
#if IPP_CLI
		static int MessageBox(IntPtr hWnd, String Text, String Caption, uint type) { return -1; }
#else
		[DllImport("user32.dll")]
		public static extern int MessageBox(IntPtr hWnd, String Text, String Caption, uint type);

		protected Dictionary<int, Bitmap> CheckStateImages = new Dictionary<int, Bitmap>();
#endif
		public static string IpaFilter = "iOS packaged applications (*.ipa)|*.ipa|All Files (*.*)|*.*";
		public static string CertificateRequestFilter = "Certificate Request (*.csr)|*.csr|All Files (*.*)|*.*";
		public static string MobileProvisionFilter = "Mobile provision files (*.mobileprovision)|*.mobileprovision|All files (*.*)|*.*";
		public static string CertificatesFilter = "Code signing certificates (*.cer;*.p12)|*.cer;*.p12|All files (*.*)|*.*";
		public static string KeysFilter = "Keys (*.key;*.p12)|*.key;*.p12|All files (*.*)|*.*";
		public static string JustXmlKeysFilter = "XML Public/Private Key Pair (*.key)|*.key|All Files (*.*)|*.*";
		public static string PListFilter = "Property Lists (*.plist)|*.plist|All Files (*.*)|*.*";
		public static string JustP12Certificates = "Code signing certificates (*.p12)|*.p12|All files (*.*)|*.*";


		//@TODO
		public static string ChoosingFilesToInstallDirectory = null;
		public static string ChoosingIpaDirectory = null;

		/// <summary>
		/// Shows a file dialog, preserving the current working directory of the application
		/// </summary>
		/// <param name="Filter">Filter string for the dialog</param>
		/// <param name="Title">Display title for the dialog</param>
		/// <param name="DefaultExtension">Default extension</param>
		/// <param name="StartingFilename">Initial filename (can be null)</param>
		/// <param name="SavedDirectory">Starting directory for the dialog (will be updated on a successful pick)</param>
		/// <param name="OutFilename">(out) The picked filename, or null if the dialog was cancelled </param>
		/// <returns>True on a successful pick (filename is in OutFilename)</returns>
		public static bool ShowOpenFileDialog(string Filter, string Title, string DefaultExtension, string StartingFilename, ref string SavedDirectory, out string OutFilename)
		{
#if !IPP_CLI
			// Save off the current working directory
			string CWD = Directory.GetCurrentDirectory();

			// Show the dialog
			System.Windows.Forms.OpenFileDialog OpenDialog = new System.Windows.Forms.OpenFileDialog();
			OpenDialog.DefaultExt = DefaultExtension;
			OpenDialog.FileName = (StartingFilename != null) ? StartingFilename : "";
			OpenDialog.Filter = Filter;
			OpenDialog.Title = Title;
			OpenDialog.InitialDirectory = (SavedDirectory != null) ? SavedDirectory : CWD;

			bool bDialogSucceeded = OpenDialog.ShowDialog() == DialogResult.OK;

			// Restore the current working directory
			Directory.SetCurrentDirectory(CWD);

			if (bDialogSucceeded)
			{
				SavedDirectory = Path.GetDirectoryName(OpenDialog.FileName);
				OutFilename = OpenDialog.FileName;

				return true;
			}
			else
#endif
			{
				OutFilename = null;
				return false;
			}
		}

#if !IPP_CLI
		public ToolsHub()
		{
			InitializeComponent();

			CheckStateImages.Add(0, iPhonePackager.Properties.Resources.GreyCheck);
			CheckStateImages.Add(1, iPhonePackager.Properties.Resources.YellowCheck);
			CheckStateImages.Add(2, iPhonePackager.Properties.Resources.GreenCheck);

			Text = Config.AppDisplayName + " Wizard";
		}

		public static ToolsHub CreateShowingTools()
		{
			ToolsHub Result = new ToolsHub();
			Result.tabControl1.SelectTab(Result.tabPage1);
			return Result;
		}

		private void ToolsHub_Shown(object sender, EventArgs e)
		{
			// Check the status of the various steps
			UpdateStatus();

			CreateCSRButton.Enabled =
			ImportMobileProvisionButton.Enabled =
			ImportProvisionButton2.Enabled =
			ImportCertificateButton.Enabled =
			ImportCertificateButton2.Enabled =
				!string.IsNullOrEmpty(Config.ProjectFile);
		}

		void UpdateStatus()
		{
			MobileProvision Provision;
			X509Certificate2 Cert;
			bool bOverridesExists;
			bool bNameMatch;
			CodeSignatureBuilder.FindRequiredFiles(out Provision, out Cert, out bOverridesExists, out bNameMatch, false);

			int ProvisionVal = 0;
			if (Provision != null)
			{
				ProvisionVal = 1;
				if (bNameMatch)
				{
					ProvisionVal = 2;
				}
			}
			int CertVal = 0;
			if (Cert != null)
			{
				CertVal = 2;
			}
			MobileProvisionCheck2.Image = MobileProvisionCheck.Image = CheckStateImages[ProvisionVal];
			CertificatePresentCheck2.Image = CertificatePresentCheck.Image = CheckStateImages[CertVal];
//			OverridesPresentCheck2.Image = OverridesPresentCheck.Image = CheckStateImages[bOverridesExists];

//			ReadyToPackageButton.Enabled = /*bOverridesExists && */(Provision != null) && (Cert != null);
		}

		private void CreateCSRButton_Click(object sender, EventArgs e)
		{
			GenerateSigningRequestDialog Dlg = new GenerateSigningRequestDialog();
			Dlg.ShowDialog(this);
		}
#endif

		public static void ShowError(string Message)
		{
#if IPP_CLI
			Console.WriteLine(Message);
			Environment.Exit(-1);
#else
			System.Windows.Forms.MessageBox.Show(Message, Config.AppDisplayName, MessageBoxButtons.OK, MessageBoxIcon.Error);
#endif
		}

		public static bool IsProfileForDistribution(MobileProvision Provision)
		{
			return CryptoAdapter.GetCommonNameFromCert(Provision.DeveloperCertificates[0]).IndexOf("iPhone Distribution", StringComparison.InvariantCultureIgnoreCase) >= 0;
		}

		public static void TryInstallingMobileProvision(string ProvisionFilename, bool ShowPrompt = true)
		{
			if (!String.IsNullOrEmpty(ProvisionFilename) || ShowOpenFileDialog(MobileProvisionFilter, "Choose a mobile provision to install", "mobileprovision", "", ref ChoosingFilesToInstallDirectory, out ProvisionFilename))
			{
				try
				{
					// Determine if this is a development or distribution certificate
					bool bIsDistribution = false;
					MobileProvision Provision = MobileProvisionParser.ParseFile(ProvisionFilename);
					bIsDistribution = IsProfileForDistribution(Provision);

					// use the input filename if the GameName is empty
					string DestName = string.IsNullOrEmpty(Program.GameName) ? Path.GetFileNameWithoutExtension(ProvisionFilename) : Program.GameName;

					// Copy the file into the destination location
					string EffectivePrefix = bIsDistribution ? "Distro_" : Config.SigningPrefix;
					string DestinationFilename = Path.Combine(Config.ProvisionDirectory, EffectivePrefix + DestName + ".mobileprovision");

					DestinationFilename = DestinationFilename.Replace("\\", "/");
					if (File.Exists(DestinationFilename))
					{
#if !IPP_CLI
						MobileProvision OldProvision = MobileProvisionParser.ParseFile(DestinationFilename);

						string MessagePrompt = String.Format(
							"{0} already contains a {1} mobile provision file.  Do you want to replace the provision '{2}' with '{3}'?",
							Config.BuildDirectory,
							bIsDistribution ? "distribution" : "development",
							OldProvision.ProvisionName,
							Provision.ProvisionName);

						if (ShowPrompt && System.Windows.Forms.MessageBox.Show(MessagePrompt, Config.AppDisplayName, MessageBoxButtons.YesNo, MessageBoxIcon.Question) == DialogResult.No)
						{
							return;
						}
#endif
						if (DestinationFilename != ProvisionFilename)
						{
							FileOperations.DeleteFile(DestinationFilename);
						}
					}

					if (DestinationFilename != ProvisionFilename)
					{
						FileOperations.CopyRequiredFile(ProvisionFilename, DestinationFilename);
					}
				}
				catch (Exception ex)
				{
					ShowError(String.Format("Encountered an error '{0} while trying to install a mobile provision", ex.Message));
				}
			}

		}

#if !IPP_CLI
		private void ImportMobileProvisionButton_Click(object sender, EventArgs e)
		{
			TryInstallingMobileProvision(null);
			UpdateStatus();
		}
#endif

		static private string CertToolData = "";
		static public void OutputReceivedCertToolProcessCall(Object Sender, DataReceivedEventArgs Line)
		{
			if ((Line != null) && !String.IsNullOrEmpty(Line.Data))
			{
				CertToolData += Line.Data + "\n";
			}
		}

		public static void TryInstallingCertificate_PromptForKey(string CertificateFilename, bool ShowPrompt = true)
		{
			try
			{
				if (!String.IsNullOrEmpty(CertificateFilename)
#if !IPP_CLI
					|| ShowOpenFileDialog(CertificatesFilter, "Choose a code signing certificate to import", "", "", ref ChoosingFilesToInstallDirectory, out CertificateFilename)
#endif
					)
				{
					if (Environment.OSVersion.Platform == PlatformID.MacOSX || Environment.OSVersion.Platform == PlatformID.Unix)
					{
						// run certtool y to get the currently installed certificates
						CertToolData = "";
						Process CertTool = new Process();
						CertTool.StartInfo.FileName = "/usr/bin/security";
						CertTool.StartInfo.UseShellExecute = false;
						CertTool.StartInfo.Arguments = "import \"" + CertificateFilename +"\" -k login.keychain";
						CertTool.StartInfo.RedirectStandardOutput = true;
						CertTool.OutputDataReceived += new DataReceivedEventHandler(OutputReceivedCertToolProcessCall);
						CertTool.Start();
						CertTool.BeginOutputReadLine();
						CertTool.WaitForExit();
						if (CertTool.ExitCode != 0)
						{
							// todo: provide some feedback that it failed
						}
						Console.Write(CertToolData);
					}
					else
					{
						// Load the certificate
						string CertificatePassword = "";
						X509Certificate2 Cert = null;
						try
						{
							Cert = new X509Certificate2(CertificateFilename, CertificatePassword, X509KeyStorageFlags.PersistKeySet | X509KeyStorageFlags.Exportable | X509KeyStorageFlags.MachineKeySet);
						}
						catch (System.Security.Cryptography.CryptographicException ex)
						{
#if !IPP_CLI

							// Try once with a password
							if (PasswordDialog.RequestPassword(out CertificatePassword))
							{
								Cert = new X509Certificate2(CertificateFilename, CertificatePassword, X509KeyStorageFlags.PersistKeySet | X509KeyStorageFlags.Exportable | X509KeyStorageFlags.MachineKeySet);
							}
							else
#endif
							{
								// User cancelled dialog, rethrow
								throw ex;
							}
						}

						// If the certificate doesn't have a private key pair, ask the user to provide one
						if (!Cert.HasPrivateKey)
						{
							string ErrorMsg = "Certificate does not include a private key and cannot be used to code sign";

							// Prompt for a key pair
							if (MessageBox(new IntPtr(0), "Next, please choose the key pair that you made when generating the certificate request.",
								Config.AppDisplayName,
								0x00000000 | 0x00000040 | 0x00001000 | 0x00010000) == 1)
							{
								string KeyFilename;
								if (ShowOpenFileDialog(KeysFilter, "Choose the key pair that belongs with the signing certificate", "", "", ref ChoosingFilesToInstallDirectory, out KeyFilename))
								{
									Cert = CryptoAdapter.CombineKeyAndCert(CertificateFilename, KeyFilename);
										
									if (Cert.HasPrivateKey)
									{
										ErrorMsg = null;
									}
								}
							}

							if (ErrorMsg != null)
							{
								throw new Exception(ErrorMsg);
							}
						}

						// Add the certificate to the store
						X509Store Store = new X509Store();
						Store.Open(OpenFlags.ReadWrite);
						Store.Add(Cert);
						Store.Close();
					}
				}
			}
			catch (Exception ex)
			{
				string ErrorMsg = String.Format("Failed to load or install certificate due to an error: '{0}'", ex.Message);
				Program.Error(ErrorMsg);
				System.Threading.Thread.Sleep(500);
				MessageBox(new IntPtr(0), ErrorMsg, Config.AppDisplayName, 0x00000000 | 0x00000010 | 0x00001000 | 0x00010000);
			}
		}

#if !IPP_CLI
		private void ImportCertificateButton_Click(object sender, EventArgs e)
		{
			TryInstallingCertificate_PromptForKey(null);
			UpdateStatus();
		}

		private void EditPlistButton_Click(object sender, EventArgs e)
		{
			ConfigureMobileGame Dlg = new ConfigureMobileGame();
			Dlg.ShowDialog(this);
			UpdateStatus();
		}

		private void CancelThisFormButton_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.Cancel;
			Close();
		}

		private void ReadyToPackageButton_Click(object sender, EventArgs e)
		{
			DialogResult = DialogResult.OK;
			Close();
		}

		private void InstallIPAButton_Click(object sender, EventArgs e)
		{
			string PickedFilename;
			if (ShowOpenFileDialog(IpaFilter, "Choose an IPA to install", "ipa", "", ref ChoosingIpaDirectory, out PickedFilename))
			{
				Program.ProgressDialog.OnBeginBackgroundWork = delegate
				{
					DeploymentHelper.InstallIPAOnConnectedDevices(PickedFilename);
				};
				Program.ProgressDialog.ShowDialog();
			}
		}

		private void ResignIPAButton_Click(object sender, EventArgs e)
		{
			GraphicalResignTool ResignTool = GraphicalResignTool.GetActiveInstance();
			ResignTool.TabBook.SelectTab(ResignTool.ResignPage);
			ResignTool.Show();
		}

		private void ProvisionCertToolsButton_Click(object sender, EventArgs e)
		{
// 			GraphicalResignTool ResignTool = GraphicalResignTool.GetActiveInstance();
// 			ResignTool.TabBook.SelectTab(ResignTool.ProvisionCertPage);
// 			ResignTool.Show();
		}

		private void OtherDeployToolsButton_Click(object sender, EventArgs e)
		{
			GraphicalResignTool ResignTool = GraphicalResignTool.GetActiveInstance();
			ResignTool.TabBook.SelectTab(ResignTool.DeploymentPage);
			ResignTool.Show();
		}

		void OpenHelpWebsite()
		{
			string Target = "http://udn.epicgames.com/Three/AppleiOSProvisioning.html";
			ProcessStartInfo PSI = new ProcessStartInfo(Target);
			Process.Start(PSI);
		}

		private void ToolsHub_HelpButtonClicked(object sender, CancelEventArgs e)
		{
			OpenHelpWebsite();
		}

		private void ToolsHub_KeyUp(object sender, KeyEventArgs e)
		{
			if (e.KeyCode == Keys.F1)
			{
				OpenHelpWebsite();
			}
		}
		
		private void HyperlinkClicked(object sender, LinkLabelLinkClickedEventArgs e)
		{
			string Target = (sender as LinkLabel).Tag as string;
			ProcessStartInfo PSI = new ProcessStartInfo(Target);
			Process.Start(PSI);
		}

		private void InstallIPA_Click(object sender, EventArgs e)
		{
			InstallIPAButton_Click(sender, e);
		}
#endif
	}
}