/**
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Net;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Xml;
using System.Security.Cryptography;
using Org.BouncyCastle.Crypto;
using Org.BouncyCastle.Security;
using Org.BouncyCastle.Crypto.Parameters;
using Org.BouncyCastle.Math;
using System.Security.Cryptography.X509Certificates;
using Org.BouncyCastle.X509;
using Org.BouncyCastle.Pkcs;
using System.Collections;

namespace iPhonePackager
{
	public class Utilities
	{
		/// <summary>
		/// Reads a string from a fixed-length ASCII array
		/// </summary>
		public static string ReadFixedASCII(BinaryReader SR, int Length)
		{
			byte[] StringAsBytes = SR.ReadBytes(Length);

			int StringLength = 0;
			while ((StringLength < Length) && (StringAsBytes[StringLength] != 0))
			{
				StringLength++;
			}

			return Encoding.ASCII.GetString(StringAsBytes, 0, StringLength);
		}

		/// <summary>
		/// Writes a string as a fixed-length ASCII array
		/// </summary>
		public static void WriteFixedASCII(BinaryWriter SW, string WriteOut, int Length)
		{
			// Encode back to ASCII
			byte[] StringAsBytesUnsized = Encoding.ASCII.GetBytes(WriteOut);
			int ValidByteCount = Math.Min(StringAsBytesUnsized.Length, Length);

			// Size it out to a fixed length buffer
			byte[] StringAsBytes = new byte[Length];
			Array.Copy(StringAsBytesUnsized, StringAsBytes, ValidByteCount);
			while (ValidByteCount < Length)
			{
				StringAsBytes[ValidByteCount] = 0;
				++ValidByteCount;
			}

			SW.Write(StringAsBytes);
		}

		public static byte[] CreateASCIIZ(string WriteOut)
		{
			// Encode back to ASCII
			byte[] StringAsBytesUnsized = Encoding.ASCII.GetBytes(WriteOut);
			int ValidByteCount = StringAsBytesUnsized.Length;
			int Length = ValidByteCount + 1;

			// Size it out to a fixed length buffer
			byte[] StringAsBytes = new byte[Length];
			Array.Copy(StringAsBytesUnsized, StringAsBytes, ValidByteCount);
			while (ValidByteCount < Length)
			{
				StringAsBytes[ValidByteCount] = 0;
				++ValidByteCount;
			}

			return StringAsBytes;
		}

		public static void VerifyStreamPosition(BinaryWriter SW, long StartingStreamPosition, long JustWroteCount)
		{
			long ExpectedPosition = StartingStreamPosition + JustWroteCount;
			if (SW.BaseStream.Position != ExpectedPosition)
			{
				throw new InvalidDataException(String.Format("Stream offset is not as expected, wrote too {0} data!", (SW.BaseStream.Position < ExpectedPosition) ? "little" : "much"));
			}
		 }

		/**
		 * Reads the specified environment variable
		 *
		 * @param	VarName		the environment variable to read
		 * @param	bDefault	the default value to use if missing
		 * @return	the value of the environment variable if found and the default value if missing
		 */
		public static bool GetEnvironmentVariable(string VarName, bool bDefault)
		{
			string Value = Environment.GetEnvironmentVariable(VarName);
			if (Value != null)
			{
				// Convert the string to its boolean value
				return Convert.ToBoolean(Value);
			}
			return bDefault;
		}

		/**
		 * Reads the specified environment variable
		 *
		 * @param	VarName		the environment variable to read
		 * @param	Default	the default value to use if missing
		 * @return	the value of the environment variable if found and the default value if missing
		 */
		public static string GetStringEnvironmentVariable(string VarName, string Default)
		{
			string Value = Environment.GetEnvironmentVariable(VarName);
			if (Value != null)
			{
				return Value;
			}
			return Default;
		}

		/**
		 * Reads the specified environment variable
		 *
		 * @param	VarName		the environment variable to read
		 * @param	Default	the default value to use if missing
		 * @return	the value of the environment variable if found and the default value if missing
		 */
		public static double GetEnvironmentVariable(string VarName, double Default)
		{
			string Value = Environment.GetEnvironmentVariable(VarName);
			if (Value != null)
			{
				return Convert.ToDouble(Value);
			}
			return Default;
		}

		/**
		 * Reads the specified environment variable
		 *
		 * @param	VarName		the environment variable to read
		 * @param	Default	the default value to use if missing
		 * @return	the value of the environment variable if found and the default value if missing
		 */
		public static string GetEnvironmentVariable(string VarName, string Default)
		{
			string Value = Environment.GetEnvironmentVariable(VarName);
			if (Value != null)
			{
				return Value;
			}
			return Default;
		}

		/// <summary>
		/// Runs an executable with the specified argument list and waits for it to terminate, capturing the standard output and return code
		/// </summary>
		public static int RunExecutableAndWait(string ExeName, string ArgumentList, out string StdOutResults)
		{
			// Create the process
			ProcessStartInfo PSI = new ProcessStartInfo(ExeName, ArgumentList);
			PSI.RedirectStandardOutput = true;
			PSI.UseShellExecute = false;
			PSI.CreateNoWindow = true;
			Process NewProcess = Process.Start(PSI);

			// Wait for the process to exit and grab it's output
			StdOutResults = NewProcess.StandardOutput.ReadToEnd();
			NewProcess.WaitForExit();
			return NewProcess.ExitCode;
		}

		public class PListHelper
		{
			public XmlDocument Doc;

			bool bReadOnly = false;

			public void SetReadOnly(bool bNowReadOnly)
			{
				bReadOnly = bNowReadOnly;
			}

			public PListHelper(string Source)
			{
				Doc = new XmlDocument();
				Doc.XmlResolver = null;
				Doc.LoadXml(Source);
			}

			public static PListHelper CreateFromFile(string Filename)
			{
				byte[] RawPList = File.ReadAllBytes(Filename);
				return new PListHelper(Encoding.UTF8.GetString(RawPList));
			}

			public void SaveToFile(string Filename)
			{
				File.WriteAllText(Filename, SaveToString(), Encoding.UTF8);
			}

			public PListHelper()
			{
				string EmptyFileText = 
					"<?xml version=\"1.0\" encoding=\"UTF-8\"?>\n" +
					"<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">\n" +
					"<plist version=\"1.0\">\n" +
					"<dict>\n" +
					"</dict>\n" +
					"</plist>\n";

				Doc = new XmlDocument();
				Doc.XmlResolver = null;
				Doc.LoadXml(EmptyFileText);
			}

			public XmlElement ConvertValueToPListFormat(object Value)
			{
				XmlElement ValueElement = null;
				if (Value is string)
				{
					ValueElement = Doc.CreateElement("string");
					ValueElement.InnerText = Value as string;
				}
				else if (Value is Dictionary<string, object>)
				{
					ValueElement = Doc.CreateElement("dict");
					foreach (var KVP in Value as Dictionary<string, object>)
					{
						AddKeyValuePair(ValueElement, KVP.Key, KVP.Value);
					}
				}
				else if (Value is Utilities.PListHelper)
				{
					Utilities.PListHelper PList = Value as Utilities.PListHelper;

					ValueElement = Doc.CreateElement("dict");

					XmlNode SourceDictionaryNode = PList.Doc.DocumentElement.SelectSingleNode("/plist/dict");
					foreach (XmlNode TheirChild in SourceDictionaryNode)
					{
						ValueElement.AppendChild(Doc.ImportNode(TheirChild, true));
					}
				}
				else if (Value is Array)
				{
					if (Value is byte[])
					{
						ValueElement = Doc.CreateElement("data");
						ValueElement.InnerText = Convert.ToBase64String(Value as byte[]);
					}
					else
					{
						ValueElement = Doc.CreateElement("array");
						foreach (var A in Value as Array)
						{
							ValueElement.AppendChild(ConvertValueToPListFormat(A));
						}
					}
				}
				else if (Value is IList)
				{
					ValueElement = Doc.CreateElement("array");
					foreach (var A in Value as IList)
					{
						ValueElement.AppendChild(ConvertValueToPListFormat(A));
					}
				}
				else if (Value is bool)
				{
					ValueElement = Doc.CreateElement(((bool)Value) ? "true" : "false");
				}
				else if (Value is double)
				{
					ValueElement = Doc.CreateElement("real");
					ValueElement.InnerText = ((double)Value).ToString();
				}
				else if (Value is int)
				{
					ValueElement = Doc.CreateElement("integer");
					ValueElement.InnerText = ((int)Value).ToString();
				}
				else
				{
					throw new InvalidDataException(String.Format("Object '{0}' is in an unknown type that cannot be converted to PList format", Value));
				}

				return ValueElement;

			}

			public void AddKeyValuePair(XmlNode DictRoot, string KeyName, object Value)
			{
				if (bReadOnly)
				{
					throw new AccessViolationException("PList has been set to read only and may not be modified");
				}

				XmlElement KeyElement = Doc.CreateElement("key");
				KeyElement.InnerText = KeyName;
				
				DictRoot.AppendChild(KeyElement);
				DictRoot.AppendChild(ConvertValueToPListFormat(Value));
			}

			public void AddKeyValuePair(string KeyName, object Value)
			{
				XmlNode DictRoot = Doc.DocumentElement.SelectSingleNode("/plist/dict");

				AddKeyValuePair(DictRoot, KeyName, Value);
			}

			/// <summary>
			/// Clones a dictionary from an existing .plist into a new one.  Root should point to the dict key in the source plist.
			/// </summary>
			public static PListHelper CloneDictionaryRootedAt(XmlNode Root)
			{
				// Create a new empty dictionary
				PListHelper Result = new PListHelper();

				// Copy all of the entries in the source dictionary into the new one
				XmlNode NewDictRoot = Result.Doc.DocumentElement.SelectSingleNode("/plist/dict");
				foreach (XmlNode TheirChild in Root)
				{
					NewDictRoot.AppendChild(Result.Doc.ImportNode(TheirChild, true));
				}

				return Result;
			}

			public bool GetString(string Key, out string Value)
			{
				string PathToValue = String.Format("/plist/dict/key[.='{0}']/following-sibling::string[1]", Key);

				XmlNode ValueNode = Doc.DocumentElement.SelectSingleNode(PathToValue);
				if (ValueNode == null)
				{
					Value = "";
					return false;
				}

				Value = ValueNode.InnerText;
				return true;
			}

			public bool GetDate(string Key, out string Value)
			{
				string PathToValue = String.Format("/plist/dict/key[.='{0}']/following-sibling::date[1]", Key);

				XmlNode ValueNode = Doc.DocumentElement.SelectSingleNode(PathToValue);
				if (ValueNode == null)
				{
					Value = "";
					return false;
				}

				Value = ValueNode.InnerText;
				return true;
			}

			public bool GetBool(string Key)
			{
				string PathToValue = String.Format("/plist/dict/key[.='{0}']/following-sibling::node()", Key);

				XmlNode ValueNode = Doc.DocumentElement.SelectSingleNode(PathToValue);
				if (ValueNode == null)
				{
					return false;
				}

				return ValueNode.Name == "true";
			}

			public delegate void ProcessOneNodeEvent(XmlNode ValueNode);

			public void ProcessValueForKey(string Key, string ExpectedValueType, ProcessOneNodeEvent ValueHandler)
			{
				string PathToValue = String.Format("/plist/dict/key[.='{0}']/following-sibling::{1}[1]", Key, ExpectedValueType);

				XmlNode ValueNode = Doc.DocumentElement.SelectSingleNode(PathToValue);
				if (ValueNode != null)
				{
					ValueHandler(ValueNode);
				}
			}

			/// <summary>
			/// Merge two plists together.  Whenever both have the same key, the value in the dominant source list wins.
			/// This is special purpose code, and only handles things inside of the <dict> tag
			/// </summary>
			public void MergePlistIn(string DominantPlist, HashSet<string> WeakKeysToKeep=null)
			{
				if (bReadOnly)
				{
					throw new AccessViolationException("PList has been set to read only and may not be modified");
				}

				XmlDocument Dominant = new XmlDocument();
				Dominant.XmlResolver = null;
				Dominant.LoadXml(DominantPlist);

				XmlNode DictionaryNode = Doc.DocumentElement.SelectSingleNode("/plist/dict");

				// Merge any key-value pairs in the strong .plist into the weak .plist
				XmlNodeList StrongKeys = Dominant.DocumentElement.SelectNodes("/plist/dict/key");
				foreach (XmlNode StrongKeyNode in StrongKeys)
				{
					string StrongKey = StrongKeyNode.InnerText;

					XmlNode WeakNode = Doc.DocumentElement.SelectSingleNode(String.Format("/plist/dict/key[.='{0}']", StrongKey));
					if (WeakNode == null)
					{
						// Doesn't exist in dominant plist, inject key-value pair
						DictionaryNode.AppendChild(Doc.ImportNode(StrongKeyNode, true));
						DictionaryNode.AppendChild(Doc.ImportNode(StrongKeyNode.NextSibling, true));
					}
					// don't overwrite values we want to keep
					else if (WeakKeysToKeep == null || !WeakKeysToKeep.Contains(StrongKey))
					{
						// Remove the existing value node from the weak file
						WeakNode.ParentNode.RemoveChild(WeakNode.NextSibling);

						// Insert a clone of the dominant value node
						WeakNode.ParentNode.InsertAfter(Doc.ImportNode(StrongKeyNode.NextSibling, true), WeakNode);
					}
				}
			}

			/// <summary>
			/// Returns each of the entries in the value tag of type array for a given key
			/// If the key is missing, an empty array is returned.
			/// Only entries of a given type within the array are returned.
			/// </summary>
			public List<string> GetArray(string Key, string EntryType)
			{
				List<string> Result = new List<string>();

				ProcessValueForKey(Key, "array", 
					delegate(XmlNode ValueNode)
					{
						foreach (XmlNode ChildNode in ValueNode.ChildNodes)
						{
							if (EntryType == ChildNode.Name)
							{
								string Value = ChildNode.InnerText;
								Result.Add(Value);
							}
						}
					});

				return Result;
			}

			/// <summary>
			/// Returns true if the key exists (and has a value) and false otherwise
			/// </summary>
			public bool HasKey(string KeyName)
			{
				string PathToKey = String.Format("/plist/dict/key[.='{0}']", KeyName);

				XmlNode KeyNode = Doc.DocumentElement.SelectSingleNode(PathToKey);
				return (KeyNode != null);
			}

			public void RemoveKeyValue(string KeyName)
			{
				if (bReadOnly)
				{
					throw new AccessViolationException("PList has been set to read only and may not be modified");
				}

				XmlNode DictionaryNode = Doc.DocumentElement.SelectSingleNode("/plist/dict");

				string PathToKey = String.Format("/plist/dict/key[.='{0}']", KeyName);
				XmlNode KeyNode = Doc.DocumentElement.SelectSingleNode(PathToKey);
				if (KeyNode != null && KeyNode.ParentNode != null)
				{
					XmlNode ValueNode = KeyNode.NextSibling;
					//remove value
					if (ValueNode != null)
					{
						ValueNode.RemoveAll();
						ValueNode.ParentNode.RemoveChild(ValueNode);
					}
					//remove key
					KeyNode.RemoveAll();
					KeyNode.ParentNode.RemoveChild(KeyNode);
				}
			}

			public void SetValueForKey(string KeyName, object Value)
			{
				if (bReadOnly)
				{
					throw new AccessViolationException("PList has been set to read only and may not be modified");
				}

				XmlNode DictionaryNode = Doc.DocumentElement.SelectSingleNode("/plist/dict");

				string PathToKey = String.Format("/plist/dict/key[.='{0}']", KeyName);
				XmlNode KeyNode = Doc.DocumentElement.SelectSingleNode(PathToKey);

				XmlNode ValueNode = null;
				if (KeyNode != null)
				{
					ValueNode = KeyNode.NextSibling;
				}

				if (ValueNode == null)
				{
					KeyNode = Doc.CreateNode(XmlNodeType.Element, "key", null);
					KeyNode.InnerText = KeyName;

					ValueNode = ConvertValueToPListFormat(Value);

					DictionaryNode.AppendChild(KeyNode);
					DictionaryNode.AppendChild(ValueNode);
				}
				else
				{
					// Remove the existing value and create a new one
					ValueNode.ParentNode.RemoveChild(ValueNode);
					ValueNode = ConvertValueToPListFormat(Value);

					// Insert the value after the key
					DictionaryNode.InsertAfter(ValueNode, KeyNode);
				}
			}

			public void SetString(string Key, string Value)
			{
				SetValueForKey(Key, Value);
			}

			public string SaveToString()
			{
				// Convert the XML back to text in the same style as the original .plist
				StringBuilder TextOut = new StringBuilder();

				// Work around the fact it outputs the wrong encoding by default (and set some other settings to get something similar to the input file)
				TextOut.Append("<?xml version=\"1.0\" encoding=\"UTF-8\"?>\n");
				XmlWriterSettings Settings = new XmlWriterSettings();
				Settings.Indent = true;
				Settings.IndentChars = "\t";
				Settings.NewLineChars = "\n";
				Settings.NewLineHandling = NewLineHandling.Replace;
				Settings.OmitXmlDeclaration = true;
				Settings.Encoding = new UTF8Encoding(false);

				// Work around the fact that it embeds an empty declaration list to the document type which codesign dislikes...
				// Replacing InternalSubset with null if it's empty.  The property is readonly, so we have to reconstruct it entirely
				Doc.ReplaceChild(Doc.CreateDocumentType(
					Doc.DocumentType.Name,
					Doc.DocumentType.PublicId,
					Doc.DocumentType.SystemId,
					String.IsNullOrEmpty(Doc.DocumentType.InternalSubset) ? null : Doc.DocumentType.InternalSubset),
					Doc.DocumentType);
				
				XmlWriter Writer = XmlWriter.Create(TextOut, Settings);

				Doc.Save(Writer);

			   // Remove the space from any standalone XML elements because the iOS parser does not handle them
			   return Regex.Replace(TextOut.ToString(), @"<(?<tag>\S+) />", "<${tag}/>");
			}
		}

		static public string GetStringFromPList(string KeyName)
		{
			// Open the .plist and read out the specified key
			string PListAsString;
			if (!Utilities.GetSourcePList(out PListAsString))
			{
				Program.Error("Failed to find source PList");
				Program.ReturnCode = (int)ErrorCodes.Error_InfoPListNotFound;
				return "(unknown)";
			}

			PListHelper Helper = new PListHelper(PListAsString);

			string Result;
			if (Helper.GetString(KeyName, out Result))
			{
				return Result;
			}
			else
			{
				Program.Error("Failed to find a value for {0} in PList", KeyName);
				Program.ReturnCode = (int)ErrorCodes.Error_KeyNotFoundInPList;
				return "(unknown)";
			}
		}

		static public string GetPrecompileSourcePListFilename()
		{
			// check for one in the project directory
			string SourceName = FileOperations.FindPrefixedFile(Config.IntermediateDirectory, Program.GameName + "-Info.plist");

			if (!File.Exists(SourceName))
			{
				// Check for a premade one
				SourceName = FileOperations.FindPrefixedFile(Config.BuildDirectory, Program.GameName + "-Info.plist");

				if (!File.Exists(SourceName))
				{
					// fallback to the shared one
					SourceName = FileOperations.FindPrefixedFile(Config.EngineBuildDirectory, "UnrealGame-Info.plist");

					if (!File.Exists(SourceName))
					{
						Program.Log("Failed to find " + Program.GameName + "-Info.plist. Please create new .plist or copy a base .plist from a provided game sample.");
					}
				}
			}

			return SourceName;
		}

		/**
		 * Handle grabbing the initial plist
		 */
		static public bool GetSourcePList(out string PListSource)
		{
			// Check for a premade one
			string SourceName = GetPrecompileSourcePListFilename();

			if (File.Exists(SourceName))
			{
				Program.Log(" ... reading source .plist: " + SourceName);
				PListSource = File.ReadAllText(SourceName);
				return true;
			}
			else
			{
				Program.Error(" ... failed to locate the source .plist file");
				Program.ReturnCode = (int)ErrorCodes.Error_KeyNotFoundInPList;
				PListSource = "";
				return false;
			}
		}
	}


	class VersionUtilities
	{
		static string RunningVersionFilename
		{
			get { return Path.Combine(Config.BuildDirectory, Program.GameName + ".PackageVersionCounter"); }
		}

		/// <summary>
		/// Reads the GameName.PackageVersionCounter from disk and bumps the minor version number in it
		/// </summary>
		/// <returns></returns>
		public static string ReadRunningVersion()
		{
			string CurrentVersion = "0.0";
			if (File.Exists(RunningVersionFilename))
			{
				CurrentVersion = File.ReadAllText(RunningVersionFilename);
			}

			return CurrentVersion;
		}

		/// <summary>
		/// Pulls apart a version string of one of the two following formats:
		///	  "7301.15 11-01 10:28"   (Major.Minor Date Time)
		///	  "7486.0"  (Major.Minor)
		/// </summary>
		/// <param name="CFBundleVersion"></param>
		/// <param name="VersionMajor"></param>
		/// <param name="VersionMinor"></param>
		/// <param name="TimeStamp"></param>
		public static void PullApartVersion(string CFBundleVersion, out int VersionMajor, out int VersionMinor, out string TimeStamp)
		{
			// Expecting source to be like "7301.15 11-01 10:28" or "7486.0"
			string[] Parts = CFBundleVersion.Split(new char[] { ' ' });

			// Parse the version string
			string[] VersionParts = Parts[0].Split(new char[] { '.' });

			if (!int.TryParse(VersionParts[0], out VersionMajor))
			{
				VersionMajor = 0;
			}

			if ((VersionParts.Length < 2) || (!int.TryParse(VersionParts[1], out VersionMinor)))
			{
				VersionMinor = 0;
			}

			TimeStamp = "";
			if (Parts.Length > 1)
			{
				TimeStamp = String.Join(" ", Parts, 1, Parts.Length - 1);
			}
		}

		public static string ConstructVersion(int MajorVersion, int MinorVersion)
		{
			return String.Format("{0}.{1}", MajorVersion, MinorVersion);
		}

		/// <summary>
		/// Parses the version string (expected to be of the form major.minor or major)
		/// Also parses the major.minor from the running version file and increments it's minor by 1.
		/// 
		/// If the running version major matches and the running version minor is newer, then the bundle version is updated.
		/// 
		/// In either case, the running version is set to the current bundle version number and written back out.
		/// </summary>
		/// <returns>The (possibly updated) bundle version</returns>
		public static string CalculateUpdatedMinorVersionString(string CFBundleVersion)
		{
			// Read the running version and bump it
			int RunningMajorVersion;
			int RunningMinorVersion;

			string DummyDate;
			string RunningVersion = ReadRunningVersion();
			PullApartVersion(RunningVersion, out RunningMajorVersion, out RunningMinorVersion, out DummyDate);
			RunningMinorVersion++;

			// Read the passed in version and bump it
			int MajorVersion;
			int MinorVersion;
			PullApartVersion(CFBundleVersion, out MajorVersion, out MinorVersion, out DummyDate);
			MinorVersion++;

			// Combine them if the stub time is older
			if ((RunningMajorVersion == MajorVersion) && (RunningMinorVersion > MinorVersion))
			{
				// A subsequent cook on the same sync, the only time that we stomp on the stub version
				MinorVersion = RunningMinorVersion;
			}

			// Combine them together
			string ResultVersionString = ConstructVersion(MajorVersion, MinorVersion);

			// Update the running version file
			Directory.CreateDirectory(Path.GetDirectoryName(RunningVersionFilename));
			File.WriteAllText(RunningVersionFilename, ResultVersionString);

			return ResultVersionString;
		}

		/// <summary>
		/// Updates the minor version in the CFBundleVersion key of the specified PList if this is a new package.
		/// Also updates the key EpicAppVersion with the bundle version and the current date/time (no year)
		/// </summary>
		public static void UpdateMinorVersion(Utilities.PListHelper PList)
		{
			string CFBundleVersion;
			if (PList.GetString("CFBundleVersion", out CFBundleVersion))
			{
				string UpdatedValue = CalculateUpdatedMinorVersionString(CFBundleVersion);
				Program.Log("Found CFBundleVersion string '{0}' and updated it to '{1}'", CFBundleVersion, UpdatedValue);
				PList.SetString("CFBundleVersion", UpdatedValue);
			}
			else
			{
				CFBundleVersion = "0.0";
			}

			// Write a second key with the packaging date/time in it
			string PackagingTime = DateTime.Now.ToString(@"MM-dd HH:mm");
			PList.SetString("EpicAppVersion", CFBundleVersion + " " + PackagingTime);
		}
	}

	class CryptoAdapter
	{
		public static AsymmetricCipherKeyPair GenerateBouncyKeyPair()
		{
			// Construct a new public/private key pair (RSA 2048 bits)
			IAsymmetricCipherKeyPairGenerator KeyGen = GeneratorUtilities.GetKeyPairGenerator("RSA");
			RsaKeyGenerationParameters KeyParams = new RsaKeyGenerationParameters(BigInteger.ValueOf(0x10001), new SecureRandom(), 2048, 25);
			KeyGen.Init(KeyParams);

			AsymmetricCipherKeyPair KeyPair = KeyGen.GenerateKeyPair();

			return KeyPair;
		}


		public static RSACryptoServiceProvider LoadKeyPairFromPKCS12Store(string Filename)
		{
			Pkcs12Store Store = new Pkcs12StoreBuilder().Build();
			Store.Load(File.OpenRead(Filename), new char[0]);

			foreach (string Alias in Store.Aliases)
			{
				if (Store.IsKeyEntry(Alias))
				{
					Console.WriteLine("Key with alias {0} is {1}", Alias, Store.GetKey(Alias).Key);
					return ConvertBouncyKeyPairToNET(Store.GetKey(Alias).Key as RsaPrivateCrtKeyParameters);
				}
			}

			return null;
		}

		public static Org.BouncyCastle.X509.X509Certificate LoadBouncyCertFromPKCS12Store(string Filename)
		{
			Pkcs12Store Store = new Pkcs12StoreBuilder().Build();
			Store.Load(File.OpenRead(Filename), new char[0]);

			foreach (string Alias in Store.Aliases)
			{
				if (Store.IsCertificateEntry(Alias))
				{
					return Store.GetCertificate(Alias).Certificate;
				}
			}

			return null;
		}

		public static RSACryptoServiceProvider LoadKeyPairFromDiskNET(string Filename)
		{
			CspParameters Setup = new CspParameters();
			Setup.KeyContainerName = "MyKeyContainer";

			RSACryptoServiceProvider KeyPair = new RSACryptoServiceProvider(Setup);
			KeyPair.PersistKeyInCsp = true;

			byte[] FileData = null;
			try
			{
				FileData = File.ReadAllBytes(Filename);
				KeyPair.FromXmlString(Encoding.UTF8.GetString(FileData));
			}
			catch (Exception)
			{
				try
				{
					KeyPair.ImportCspBlob(FileData);
				}
				catch (Exception)
				{
					try
					{
						return LoadKeyPairFromPKCS12Store(Filename);
					}
					catch (Exception)
					{
						return null;
					}
				}
			}

			return KeyPair;
		}

		public static AsymmetricCipherKeyPair LoadKeyPairFromDiskBouncy(string Filename)
		{
			RSACryptoServiceProvider KeyNET = LoadKeyPairFromDiskNET(Filename);
			if (KeyNET != null)
			{
				return ConvertNETKeyPairToBouncy(KeyNET);
			}
			else
			{
				return null;
			}
		}

		public static AsymmetricCipherKeyPair ConvertNETKeyPairToBouncy(RSACryptoServiceProvider KeyPair)
		{
			return DotNetUtilities.GetKeyPair(KeyPair);
		}

		public static RSACryptoServiceProvider ConvertBouncyKeyPairToNET(AsymmetricCipherKeyPair KeyPair)
		{
			RsaPrivateCrtKeyParameters PrivateKeyInfo = KeyPair.Private as RsaPrivateCrtKeyParameters;

			return ConvertBouncyKeyPairToNET(PrivateKeyInfo);
		}

		public static RSACryptoServiceProvider ConvertBouncyKeyPairToNET(RsaPrivateCrtKeyParameters PrivateKeyInfo)
		{
			CspParameters CSPParams = new CspParameters();
			CSPParams.KeyContainerName = "KeyContainer";
			RSACryptoServiceProvider Result = new RSACryptoServiceProvider(2048, CSPParams);

			RSAParameters PrivateKeyInfoDotNET = DotNetUtilities.ToRSAParameters(PrivateKeyInfo);
			Result.ImportParameters(PrivateKeyInfoDotNET);

			return Result;
		}

		public static string GetCommonNameFromCert(X509Certificate2 Cert)
		{
			// Make sure we have a useful friendly name
			string CommonName = "(no common name present)";

			string FullName = Cert.SubjectName.Name;
			char[] SplitChars = { ',' };
			string[] NameParts = FullName.Split(SplitChars);

			foreach (string Part in NameParts)
			{
				string CleanPart = Part.Trim();
				if (CleanPart.StartsWith("CN="))
				{
				   CommonName = CleanPart.Substring(3);
				}
			}

			return CommonName;
		}

		public static string GetFriendlyNameFromCert(X509Certificate2 Cert)
		{
			if (Environment.OSVersion.Platform == PlatformID.Unix || Environment.OSVersion.Platform == PlatformID.MacOSX)
			{
				return GetCommonNameFromCert(Cert);
			}
			else
			{
				// Make sure we have a useful friendly name
				string FriendlyName = Cert.FriendlyName;
				if ((FriendlyName == "") || (FriendlyName == null))
				{
					FriendlyName = GetCommonNameFromCert(Cert);
				}

				return FriendlyName;
			}
		}

	/// <summary>
	/// Merges a certificate and private key into a single combined certificate
	/// </summary>
	public static X509Certificate2 CombineKeyAndCert(string CertificateFilename, string KeyFilename)
		{
			// Load the certificate
			string CertificatePassword = "";
			X509Certificate2 Cert = new X509Certificate2(CertificateFilename, CertificatePassword, X509KeyStorageFlags.PersistKeySet | X509KeyStorageFlags.Exportable | X509KeyStorageFlags.MachineKeySet);

			// Make sure we have a useful friendly name
			string FriendlyName = GetFriendlyNameFromCert(Cert);

			// Create a PKCS#12 store with both the certificate and the private key in it
			Pkcs12Store Store = new Pkcs12StoreBuilder().Build();

			X509CertificateEntry[] CertChain = new X509CertificateEntry[1];
			Org.BouncyCastle.X509.X509Certificate BouncyCert = DotNetUtilities.FromX509Certificate(Cert);
			CertChain[0] = new X509CertificateEntry(BouncyCert);

			AsymmetricCipherKeyPair KeyPair = LoadKeyPairFromDiskBouncy(KeyFilename);
			if (KeyPair == null || KeyPair.Private == null)
			{
				throw new InvalidDataException("The key pair provided does contain a private key.  Make sure you provide the same key pair that was used to generate the original certificate signing request");
			}

			Store.SetKeyEntry(FriendlyName, new AsymmetricKeyEntry(KeyPair.Private), CertChain);

			// Verify the public key from the key pair matches the certificate's public key
			AsymmetricKeyParameter CertPublicKey = BouncyCert.GetPublicKey();
			if (!(KeyPair.Public as RsaKeyParameters).Equals(CertPublicKey as RsaKeyParameters))
			{
				throw new InvalidDataException("The key pair provided does not match the certificate.  Make sure you provide the same key pair that was used to generate the original certificate signing request");
			}

			// Export the merged cert as a .p12
			string TempFileName = Path.GetTempFileName();

			string ReexportedPassword = "password";

			Stream OutStream = File.OpenWrite(TempFileName);
			Store.Save(OutStream, ReexportedPassword.ToCharArray(), new Org.BouncyCastle.Security.SecureRandom());
			OutStream.Close();

			// Load it back in and delete the temporary file
			X509Certificate2 Result = new X509Certificate2(TempFileName, ReexportedPassword, X509KeyStorageFlags.Exportable | X509KeyStorageFlags.PersistKeySet | X509KeyStorageFlags.MachineKeySet);
			FileOperations.DeleteFile(TempFileName);
			return Result;
		}
	}
}



