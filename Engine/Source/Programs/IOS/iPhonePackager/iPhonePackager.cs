/**
 * Copyright Epic Games, Inc. All Rights Reserved.
 */

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Net;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
#if !IPP_CLI
using System.Windows.Forms;
#endif
using Ionic.Zlib;
using System.ComponentModel;
using System.Security.Cryptography.X509Certificates;
using Microsoft.Win32;

namespace iPhonePackager
{
    // NOTE: this needs to be kept in sync with EditorAnalytics.h and AutomationTool/Program.cs
    public enum ErrorCodes
    {
        Error_UATNotFound = -1,
        Error_Success = 0,
        Error_Unknown = 1,
        Error_Arguments = 2,
        Error_UnknownCommand = 3,
        Error_SDKNotFound = 10,
        Error_ProvisionNotFound = 11,
        Error_CertificateNotFound = 12,
        Error_ProvisionAndCertificateNotFound = 13,
        Error_InfoPListNotFound = 14,
        Error_KeyNotFoundInPList = 15,
        Error_ProvisionExpired = 16,
        Error_CertificateExpired = 17,
        Error_CertificateProvisionMismatch = 18,
        Error_CodeUnsupported = 19,
        Error_PluginsUnsupported = 20,
        Error_UnknownCookFailure = 25,
        Error_UnknownDeployFailure = 26,
        Error_UnknownBuildFailure = 27,
        Error_UnknownPackageFailure = 28,
        Error_UnknownLaunchFailure = 29,
        Error_StageMissingFile = 30,
        Error_FailedToCreateIPA = 31,
        Error_FailedToCodeSign = 32,
        Error_DeviceBackupFailed = 33,
        Error_AppUninstallFailed = 34,
        Error_AppInstallFailed = 35,
        Error_AppNotFound = 36,
        Error_StubNotSignedCorrectly = 37,
        Error_IPAMissingInfoPList = 38,
        Error_DeleteFile = 39,
        Error_DeleteDirectory = 40,
        Error_CreateDirectory = 41,
        Error_CopyFile = 42,
        Error_OnlyOneObbFileSupported = 50,
        Error_FailureGettingPackageInfo = 51,
        Error_OnlyOneTargetConfigurationSupported = 52,
        Error_ObbNotFound = 53,
        Error_AndroidBuildToolsPathNotFound = 54,
        Error_NoApkSuitableForArchitecture = 55,
        Error_FilesInstallFailed = 56,
        Error_RemoteCertificatesNotFound = 57,
        Error_LauncherFailed = 100,
        Error_UATLaunchFailure = 101,
        Error_FailedToDeleteStagingDirectory = 102,
        Error_MissingExecutable = 103,
        Error_DeviceNotSetupForDevelopment = 150,
        Error_DeviceOSNewerThanSDK = 151,
        Error_TestFailure = 152,
        Error_SymbolizedSONotFound = 153,
        Error_LicenseNotAccepted = 154,
        Error_AndroidOBBError = 155,
		Error_SDKInstallFailed = 200,
		Error_DeviceUpdateFailed = 201,
	};

    public partial class Program
    {
        static public int ReturnCode = 0;
        static string MainCommand = "";
        static string MainRPCCommand = "";
        static public string GameName = "";
        static private string GamePath = "";
        static public string GameConfiguration = "";
        static public string Architecture = "";
        static public string SchemeName = "";
        static public string SchemeConfiguration = "";

#if !IPP_CLI
		static public SlowProgressDialog ProgressDialog = null;
#endif
		static public BackgroundWorker BGWorker = null;
        static public int ProgressIndex = 0;
        static public string AdditionalCommandline = "";
        static public bool IsClient = false;

        static public void UpdateStatus(string Line)
        {
            if (BGWorker != null)
            {
                int Percent = Math.Min(Math.Max(ProgressIndex, 0), 100);
                BGWorker.ReportProgress(Percent, Line);
            }
        }

        static public void Log(string Line)
        {
            UpdateStatus(Line);
            Console.WriteLine(Line);
        }

        static public void Log(string Line, params object[] Args)
        {
            Log(String.Format(Line, Args));
        }

        static public void LogVerbose(string Line)
        {
            if (Config.bVerbose)
            {
                UpdateStatus(Line);
                Console.WriteLine(Line);
            }
        }

        static public void LogVerbose(string Line, params object[] Args)
        {
            LogVerbose(String.Format(Line, Args));
        }

        static public void Error(string Line, int Code = 1)
        {
            if (Program.ReturnCode == 0)
            {
                Program.ReturnCode = Code;
            }

            Console.ForegroundColor = ConsoleColor.Red;
            Log("IPP ERROR: " + Line);

            Console.ResetColor();
        }

        static public void Error(string Line, params object[] Args)
        {
            Error(String.Format(Line, Args));
        }

        static public void Warning(string Line)
        {
            Console.ForegroundColor = ConsoleColor.Yellow;
            Log("IPP WARNING: " + Line);
            Console.ResetColor();
        }

        static public void Warning(string Line, params object[] Args)
        {
            Warning(String.Format(Line, Args));
        }

        static private bool ParseCommandLine(ref string[] Arguments)
        {
            if (Arguments.Length == 0)
            {
#if !IPP_CLI
				StartVisuals();

                // we NEED a project, so show a uproject picker
                string UProjectFile;
                string StartingDir = "";

				if (ToolsHub.ShowOpenFileDialog("Unreal Project Files (*.uproject)|*.uproject;", "IPhonePackager now requires a .uproject file for certificate/provisioning setup", "mobileprovision", "", ref StartingDir, out UProjectFile))
                {
                    Arguments = new string[] { UProjectFile };
                }
                else
                {
                    Arguments = new string[] { "gui" };
                }
#endif
			}

			if (Arguments.Length == 1)
            {
                // if the only argument is a uproject, then assume gui mode, with the uproject as the project
                if (Arguments[0].EndsWith(".uproject"))
                {
                    Config.ProjectFile = GamePath = Arguments[0];

                    MainCommand = "gui";
                }
                else
                {
                    MainCommand = Arguments[0];
                }
            }
            else if (Arguments.Length == 2)
            {
                MainCommand = Arguments[0];
                GamePath = Arguments[1];
                if (GamePath.EndsWith(".uproject"))
                {
                    Config.ProjectFile = GamePath;
                }
            }
            else if (Arguments.Length >= 2)
            {
                MainCommand = Arguments[0];
                GamePath = Arguments[1];
                if (GamePath.EndsWith(".uproject"))
                {
                    Config.ProjectFile = GamePath;
                }

                for (int ArgIndex = 2; ArgIndex < Arguments.Length; ArgIndex++)
                {
                    string Arg = Arguments[ArgIndex].ToLowerInvariant();

                    if (Arg.StartsWith("-"))
                    {
                        // Behavior switches
                        switch (Arg)
                        {
                            case "-verbose":
                                Config.bVerbose = true;
                                break;
                            case "-strip":
                                Config.bForceStripSymbols = true;
                                break;
                            case "-compress=best":
                                Config.RecompressionSetting = (int)CompressionLevel.BestCompression;
                                break;
                            case "-compress=fast":
                                Config.RecompressionSetting = (int)CompressionLevel.BestSpeed;
                                break;
                            case "-compress=none":
                                Config.RecompressionSetting = (int)CompressionLevel.None;
                                break;
                            case "-distribution":
                                Config.bForDistribution = true;
                                break;
                            case "-codebased":
                                Config.bIsCodeBasedProject = true;
                                break;
                            case "-createstub":
                                Config.bCreateStubSet = true;
                                break;
                            case "-sign":
                                Config.bPerformResignWhenRepackaging = true;
                                break;
                            case "-cookonthefly":
                                Config.bCookOnTheFly = true;
                                break;
                            case "-iterate":
                                Config.bIterate = true;
                                break;
                            case "-tvos":
                                Config.OSString = "TVOS";
                                break;
                            case "-autosigning":
                                Config.bAutomaticSigning = true;
                                break;
                        }

                        // get the stage dir path
                        if (Arg == "-stagedir")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.RepackageStagingDirectory = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        // get the provisioning uuid
                        else if (Arg == "-provisioninguuid")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.ProvisionUUID = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-provisionfile")
                        {
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.ProvisionFile = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-teamID")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.TeamID = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-manifest")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.DeltaManifest = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-backup")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.FilesForBackup.Add(Arguments[++ArgIndex]);
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-config")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                GameConfiguration = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        // append a name to the bungle identifier and display name
                        else if (Arg == "-bundlename")
                        {
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                string projectName = "[PROJECT_NAME]";
                                string bundleId = Arguments[++ArgIndex];

                                // Check for an illegal bundle id
                                for (int i = 0; i < bundleId.Length; ++i)
                                {
                                    char c = bundleId[i];

                                    if (c == '[')
                                    {
                                        if (bundleId.IndexOf(projectName, i) != i)
                                        {
                                            Error("Illegal character in bundle ID");
                                            return false;
                                        }
                                        i += projectName.Length;
                                    }
                                    else if ((c < '0' || c > '9') && (c < 'a' || c > 'z') && (c < 'A' || c > 'Z') && c != '.' && c != '-')
                                    {
                                        Error("Illegal character in bundle ID");
                                        return false;
                                    }
                                }

                                // Save the verified bundle id
                                Config.OverrideBundleName = bundleId;
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-schemename")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                SchemeName = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-schemeconfig")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                SchemeConfiguration = Arguments[++ArgIndex];

                                if (SchemeConfiguration.EndsWith("Client"))
                                {
                                    IsClient = true;
                                }
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-mac")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.OverrideMacName = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-architecture" || Arg == "-arch")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Architecture = "-" + Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-project")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.ProjectFile = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-device")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.DeviceId = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-additionalcommandline")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                AdditionalCommandline = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-provision")
                        {
                            // make sure there's at least one more arg
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.Provision = Arguments[++ArgIndex];
                                Config.bProvision = !String.IsNullOrEmpty(Config.Provision);
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-certificate")
                        {
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.Certificate = Arguments[++ArgIndex];
                                Config.bCert = !String.IsNullOrEmpty(Config.Certificate);
                            }
                            else
                            {
                                return false;
                            }
                        }
                        else if (Arg == "-outputcertificate")
                        {
                            if (Arguments.Length > ArgIndex + 1)
                            {
                                Config.OutputCertificate = Arguments[++ArgIndex];
                            }
                            else
                            {
                                return false;
                            }
                        }
						else if (Arg == "-targetname")
						{
							// make sure there's at least one more arg
							if (Arguments.Length > ArgIndex + 1)
							{
								Config.TargetName = Arguments[++ArgIndex];
							}
							else
							{
								return false;
							}
						}
					}
					else
                    {
                        // RPC command
                        MainRPCCommand = Arguments[ArgIndex];
                    }
                }

                if (!SSHCommandHelper.ParseSSHProperties(Arguments))
                {
                    return false;
                }
            }

            return (true);
        }

        static private bool CheckArguments()
        {
            if (GameConfiguration.Length == 0)
            {
                GameConfiguration = "GameConfigurationWasNotSpecifiedToIPP";
            }

            if (GameName.Length == 0)
            {
                Error("Invalid number of arguments");
                Program.ReturnCode = (int)ErrorCodes.Error_Arguments;
                return false;
            }

            return true;
        }

        static public void ExecuteCommand(string Command, string RPCCommand)
        {
            if (ReturnCode > 0)
            {
                Warning("Error in previous command; suppressing: " + Command + " " + RPCCommand ?? "");
                return;
            }

            if (Config.bVerbose)
            {
                Log("");
                Log("----------");
                Log(String.Format("Executing command '{0}' {1}...", Command, (RPCCommand != null) ? ("'" + RPCCommand + "' ") : ""));
            }

            try
            {
                bool bHandledCommand = CookTime.ExecuteCookCommand(Command, RPCCommand);

                if (!bHandledCommand)
                {
                    bHandledCommand = CompileTime.ExecuteCompileCommand(Command, RPCCommand);
                }

                if (!bHandledCommand)
                {
                    bHandledCommand = DeploymentHelper.ExecuteDeployCommand(Command, GamePath, RPCCommand);
                }

                if (!bHandledCommand)
                {
                    bHandledCommand = true;
                    switch (Command)
                    {
#if !IPP_CLI
                        case "configure":
                            if (Config.bForDistribution)
                            {
                                Error("configure cannot be used with -distribution");
                                Program.ReturnCode = (int)ErrorCodes.Error_Arguments;
                            }
                            else
                            {
                                RunInVisualMode(delegate { return new ConfigureMobileGame(); });
                            }
                            break;
#endif
                        default:
                            bHandledCommand = false;
                            break;
                    }
                }

                if (!bHandledCommand)
                {
                    Error("Unknown command");
                    Program.ReturnCode = (int)ErrorCodes.Error_UnknownCommand;
                }
            }
            catch (Exception Ex)
            {
                Error("Error while executing command: " + Ex.ToString());
                if (Program.ReturnCode == 0)
                {
                    Program.ReturnCode = (int)ErrorCodes.Error_Unknown;
                }
            }

            Console.WriteLine();
        }

        /**
		 * Assembly resolve method to pick correct StandaloneSymbolParser DLL
		 */
        static Assembly CurrentDomain_AssemblyResolve(Object sender, ResolveEventArgs args)
        {
            // Name is fully qualified assembly definition - e.g. "p4dn, Version=1.0.0.0, Culture=neutral, PublicKeyToken=ff968dc1933aba6f"
            string[] AssemblyInfo = args.Name.Split(",".ToCharArray());
            string AssemblyName = AssemblyInfo[0];

            if (AssemblyName.ToLowerInvariant() == "unrealcontrols")
            {
                AssemblyName = Path.GetFullPath(@"..\" + AssemblyName + ".dll");

                Debug.WriteLineIf(System.Diagnostics.Debugger.IsAttached, "Loading assembly: " + AssemblyName);

                if (File.Exists(AssemblyName))
                {
                    Assembly SymParser = Assembly.LoadFile(AssemblyName);
                    return SymParser;
                }
            }
            else if (AssemblyName.ToLowerInvariant() == "rpcutility")
            {
                AssemblyName = Path.GetFullPath(@"..\" + AssemblyName + ".exe");

                Debug.WriteLineIf(System.Diagnostics.Debugger.IsAttached, "Loading assembly: " + AssemblyName);

                if (File.Exists(AssemblyName))
                {
                    Assembly SymParser = Assembly.LoadFile(AssemblyName);
                    return SymParser;
                }
            }
            else if (AssemblyName.ToLowerInvariant() == "bouncycastle.crypto")
            {
                AssemblyName = Path.GetFullPath(@"..\..\ThirdParty\IOS\" + AssemblyName + ".dll");

                Debug.WriteLineIf(System.Diagnostics.Debugger.IsAttached, "Loading assembly: " + AssemblyName);

                if (File.Exists(AssemblyName))
                {
                    Assembly A = Assembly.LoadFile(AssemblyName);
                    return A;
                }
            }
			else if (AssemblyName.ToLowerInvariant() == "sysadminslv.asn1parser")
			{
				AssemblyName = Path.GetFullPath(@"..\..\ThirdParty\IOS\" + AssemblyName + ".dll");

				Debug.WriteLineIf(System.Diagnostics.Debugger.IsAttached, "Loading assembly: " + AssemblyName);

				if (File.Exists(AssemblyName))
				{
					Assembly A = Assembly.LoadFile(AssemblyName);
					return A;
				}
			}
			else if (AssemblyName.ToLowerInvariant() == "ionic.zip.reduced")
            {
                AssemblyName = Path.GetFullPath(@"..\" + AssemblyName + ".dll");

                Debug.WriteLineIf(System.Diagnostics.Debugger.IsAttached, "Loading assembly: " + AssemblyName);

                if (File.Exists(AssemblyName))
                {
                    Assembly A = Assembly.LoadFile(AssemblyName);
                    return A;
                }
            }

            return (null);
        }

#if !IPP_CLI
		delegate Form CreateFormDelegate();

        static bool bVisualsStarted = false;
        static void StartVisuals()
        {
            if (bVisualsStarted)
            {
                return;
            }
            bVisualsStarted = true;

            // make the form look good on modern displays!
            if (Environment.OSVersion.Version.Major >= 6)
            {
                SetProcessDPIAware();
            }

            Application.EnableVisualStyles();
            Application.SetCompatibleTextRenderingDefault(false);

            ProgressDialog = new SlowProgressDialog();
        }

        [System.Runtime.InteropServices.DllImport("user32.dll")]
        private static extern bool SetProcessDPIAware();

        static void RunInVisualMode(CreateFormDelegate Work)
        {
            StartVisuals();

            Form DisplayForm = Work.Invoke();
            Application.Run(DisplayForm);
        }
#endif
		static void ListDevices()
        {
            var DeviceList = DeploymentHelper.GetAllConnectedDevices();

            Console.WriteLine("-------------------------------------------------------");
            Console.WriteLine("List of devices attached");

            foreach (var DeviceInfo in DeviceList)
            {
                string UDID = DeviceInfo.UDID;
                string DeviceName = DeviceInfo.DeviceName;

                Console.WriteLine("{0} device:{1}", UDID, DeviceName);
            }
        }

        static string FindWindowsStoreITunesDLL()
        {
            string InstallPath = null;

            string PackagesKeyName = "Software\\Classes\\Local Settings\\Software\\Microsoft\\Windows\\CurrentVersion\\AppModel\\PackageRepository\\Packages";

            RegistryKey PackagesKey = Registry.LocalMachine.OpenSubKey(PackagesKeyName);
            if (PackagesKey != null)
            {
                string[] PackageSubKeyNames = PackagesKey.GetSubKeyNames();

                foreach (string PackageSubKeyName in PackageSubKeyNames)
                {
                    if (PackageSubKeyName.Contains("AppleInc.iTunes") && (PackageSubKeyName.Contains("_x64") || PackageSubKeyName.Contains("_x86")))
                    {
                        string FullPackageSubKeyName = PackagesKeyName + "\\" + PackageSubKeyName;

                        RegistryKey iTunesKey = Registry.LocalMachine.OpenSubKey(FullPackageSubKeyName);
                        if (iTunesKey != null)
                        {
                            InstallPath = (string)iTunesKey.GetValue("Path") + "\\AMDS32\\MobileDevice.dll";
                            break;
                        }
                    }
                }
            }

            return InstallPath;
        }

        /**
		 * Main control loop
		 */
        [STAThread]
        static int Main(string[] args)
        {
            // remember the working directory at start, as the game path could be relative to this path
            string InitialCurrentDirectory = Environment.CurrentDirectory;

			// set the working directory to the location of the application (so relative paths always work)
			Environment.CurrentDirectory = Path.GetDirectoryName(
#if IPP_CLI
				System.Reflection.Assembly.GetExecutingAssembly().Location
#else
				Application.ExecutablePath
#endif
			);
            AppDomain.CurrentDomain.AssemblyResolve += new ResolveEventHandler(CurrentDomain_AssemblyResolve);

			// Force UTF-8 encoding so UBT and the engine can parse the IPP output consistently.
			Console.OutputEncoding = Encoding.UTF8;

            // A simple, top-level try-catch block
            try
            {
                if (!ParseCommandLine(ref args))
                {
                    Log("Usage: iPhonePackager <Command> <GameName> [RPCCommand &| Switch]");
                    Log("");
                    Log("Common commands:");
                    Log(" ... RepackageIPA GameName");
                    Log(" ... PackageIPA GameName");
                    Log(" ... PackageApp GameName");
                    Log(" ... Deploy PathToIPA");
                    Log(" ... RepackageFromStage GameName");
                    Log(" ... Devices");
                    Log(" ... Validate");
                    Log(" ... Install");
                    Log("");
                    Log("Configuration switches:");
                    Log("	 -stagedir <path>		  sets the directory to copy staged files from (defaults to none)");
                    Log("	 -project <path>		  path to the project being packaged");
                    Log("	 -provisioning <uuid>	  uuid of the provisioning selected");
                    Log("	 -compress=fast|best|none  packaging compression level (defaults to none)");
                    Log("	 -strip					strip symbols during packaging");
                    Log("	 -config				   game configuration (e.g., Shipping, Development, etc...)");
                    Log("	 -distribution			 packaging for final distribution");
                    Log("	 -codebased				   packaging a c++ code based project");
                    Log("	 -createstub			   packaging stub IPA for later repackaging");
                    Log("	 -mac <MacName>			overrides the machine to use for any Mac operations");
                    Log("	 -arch <Architecture>	  sets the architecture to use (blank for default, -simulator for simulator builds)");
                    Log("	 -device <DeviceID>		sets the device to install the IPA on");
                    Log("");
                    Log("Commands: RPC, Clean");
                    Log("  StageMacFiles, GetIPA, Deploy, Install, Uninstall");
                    Log("");
                    Log("RPC Commands: SetExec, InstallProvision, MakeApp, DeleteIPA, Copy, Kill, Strip, Zip, GenDSYM");
                    Log("");
                    Log("Sample commandlines:");
                    Log(" ... iPhonePackager Deploy UnrealGame Release");
                    Log(" ... iPhonePackager RPC SwordGame Shipping MakeApp");
                    return (int)ErrorCodes.Error_Arguments;
                }

                Log("Executing iPhonePackager " + String.Join(" ", args));
                Log("CWD: " + Directory.GetCurrentDirectory());
                Log("Initial Dir: " + InitialCurrentDirectory);
                Log("Env CWD: " + Environment.CurrentDirectory);

				Encoding.RegisterProvider(CodePagesEncodingProvider.Instance);

				// Ensure shipping configuration for final distributions
				if (Config.bForDistribution && (GameConfiguration != "Shipping"))
                {
                    Program.Warning("Distribution builds should be made in the Shipping configuration!");
                }

                // process the GamePath (if could be ..\Samples\MyDemo\ or ..\Samples\MyDemo\MyDemo.uproject
                GameName = Path.GetFileNameWithoutExtension(GamePath);
                if (GameName.Equals("UE4", StringComparison.InvariantCultureIgnoreCase) || GameName.Equals("Engine", StringComparison.InvariantCultureIgnoreCase))
                {
                    GameName = "UnrealGame";
                }

                // setup configuration
                if (!Config.Initialize(InitialCurrentDirectory, GamePath))
                {
                    return (int)ErrorCodes.Error_Arguments;
                }

                switch (MainCommand.ToLowerInvariant())
                {
                    case "validate":
                        // check to see if iTunes is installed
                        string dllPath = "";
                        if (Environment.OSVersion.Platform == PlatformID.MacOSX || Environment.OSVersion.Platform == PlatformID.Unix)
                        {
                            ProcessStartInfo StartInfo = new ProcessStartInfo("/usr/bin/xcode-select", "--print-path");
                            StartInfo.UseShellExecute = false;
                            StartInfo.RedirectStandardOutput = true;
                            StartInfo.CreateNoWindow = true;

                            using (Process LocalProcess = Process.Start(StartInfo))
                            {
                                StreamReader OutputReader = LocalProcess.StandardOutput;
                                // trim off any extraneous new lines, helpful for those one-line outputs
                                dllPath = OutputReader.ReadToEnd().Trim();
                            }
                        }
                        else
                        {
                            dllPath = Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\Wow6432Node\\Apple Inc.\\Apple Mobile Device Support\\Shared", "iTunesMobileDeviceDLL", null) as string;
                            if (String.IsNullOrEmpty(dllPath) || !File.Exists(dllPath))
                            {
                                dllPath = Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\Wow6432Node\\Apple Inc.\\Apple Mobile Device Support\\Shared", "MobileDeviceDLL", null) as string;
                                if (String.IsNullOrEmpty(dllPath) || !File.Exists(dllPath))
                                {
									// iTunes >= 12.7 doesn't have a key specifying the 32-bit DLL but it does have a ASMapiInterfaceDLL key and MobileDevice.dll is in usually in the same directory
									dllPath = Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\Wow6432Node\\Apple Inc.\\Apple Mobile Device Support\\Shared", "ASMapiInterfaceDLL", null) as string;
									dllPath = String.IsNullOrEmpty(dllPath) ? null : dllPath.Substring(0, dllPath.LastIndexOf('\\') + 1) + "MobileDevice.dll";

									if (String.IsNullOrEmpty(dllPath) || !File.Exists(dllPath))
									{
										dllPath = FindWindowsStoreITunesDLL();
									}
                                }
                            }
                        }
                        if (String.IsNullOrEmpty(dllPath) || (!File.Exists(dllPath) && !Directory.Exists(dllPath)))
                        {
                            Error("iTunes Not Found!!", (int)ErrorCodes.Error_SDKNotFound);
                        }
                        else
                        {
                            // validate there is a useable provision and cert
                            MobileProvision Provision;
                            X509Certificate2 Cert;
                            bool bHasOverrides;
                            bool bNameMatch;
                            bool foundPlist = CodeSignatureBuilder.FindRequiredFiles(out Provision, out Cert, out bHasOverrides, out bNameMatch);
                            if (!foundPlist)
                            {
                                Error("Could not find a valid plist file!!", (int)ErrorCodes.Error_InfoPListNotFound);
                            }
                            else if (!Config.bAutomaticSigning)
                            {
                                if (Provision == null && Cert == null)
                                {
                                    Error("No Provision or cert found!!", (int)ErrorCodes.Error_ProvisionAndCertificateNotFound);
                                }
                                else if (Provision == null)
                                {
                                    Error("No Provision found!!", (int)ErrorCodes.Error_ProvisionNotFound);
                                }
                                else if (Cert == null)
                                {
                                    Error("No Signing Certificate found!!", (int)ErrorCodes.Error_CertificateNotFound);
                                }
                            }
                            else
                            {
                                if (Config.TeamID == null)
                                {
                                    Error("No TeamID for automatic signing!!", (int)ErrorCodes.Error_ProvisionNotFound);
                                }
                            }
                        }
                        break;

                    case "packageapp":
                        if (CheckArguments())
                        {
                            if (Config.bCreateStubSet)
                            {
                                Error("packageapp cannot be used with the -createstub switch");
                                Program.ReturnCode = (int)ErrorCodes.Error_Arguments;
                            }
                            else
                            {
                                // Create the .app on the Mac
                                CompileTime.CreateApplicationDirOnMac();
                            }
                        }
                        break;

                    case "repackagefromstage":
                        if (CheckArguments())
                        {
                            if (Config.bCreateStubSet)
                            {
                                Error("repackagefromstage cannot be used with the -createstub switches");
                                Program.ReturnCode = (int)ErrorCodes.Error_Arguments;
                            }
                            else
                            {
                                bool bProbablyCreatedStub = Utilities.GetEnvironmentVariable("ue.IOSCreateStubIPA", true);
                                if (!bProbablyCreatedStub)
                                {
                                    Warning("ue.IOSCreateStubIPA is currently FALSE, which means you may be repackaging with an out of date stub IPA!");
                                }

                                CookTime.RepackageIPAFromStub();
                            }
                        }
                        break;

                    // this is the "super fast just move executable" mode for quick programmer iteration
                    case "dangerouslyfast":
                        if (CheckArguments())
                        {
                            CompileTime.DangerouslyFastMode();
                        }
                        break;

                    case "packageipa":
                        if (CheckArguments())
                        {
                            CompileTime.PackageIPAOnMac();
                        }
                        break;

                    case "install":
                        GameName = "";
                        if (Config.bProvision)
                        {
                            ToolsHub.TryInstallingMobileProvision(Config.Provision, false);
                        }
                        if (Config.bCert)
                        {
                            ToolsHub.TryInstallingCertificate_PromptForKey(Config.Certificate, false);
                        }
                        CodeSignatureBuilder.FindCertificates();
                        CodeSignatureBuilder.FindProvisions(Config.OverrideBundleName);
                        break;

                    case "certificates":
                        {
                            CodeSignatureBuilder.FindCertificates();
                            CodeSignatureBuilder.FindProvisions(Config.OverrideBundleName);
                        }
                        break;

#if !IPP_CLI
					case "resigntool":
                        RunInVisualMode(delegate { return new GraphicalResignTool(); });
                        break;

                    case "certrequest":
                        RunInVisualMode(delegate { return new GenerateSigningRequestDialog(); });
                        break;

                    case "gui":
                        RunInVisualMode(delegate { return ToolsHub.CreateShowingTools(); });
                        break;
#endif
                    case "devices":
                        ListDevices();
                        break;

                    case "signing_match":
                        {
                            MobileProvision Provision;
                            X509Certificate2 Cert;
                            bool bNameMatch;
                            bool bHasOverrideFile;
                            MobileProvision.CacheMobileProvisions();
                            if (CodeSignatureBuilder.FindRequiredFiles(out Provision, out Cert, out bHasOverrideFile, out bNameMatch) && Cert != null)
                            {
                                // print out the provision and cert name
                                Program.LogVerbose("CERTIFICATE-{0},PROVISION-{1}", CryptoAdapter.GetFriendlyNameFromCert(Cert), Provision.FileName);
                            }
                            else
                            {
                                Program.LogVerbose("No matching Signing Data found!");
                            }
                        }
                        break;

					case "cleanprovisions":
						MobileProvision.CleanMobileProvisions();
						break;

					default:
                        // Commands by themself default to packaging for the device
                        if (CheckArguments())
                        {
                            ExecuteCommand(MainCommand, MainRPCCommand);
                        }
                        break;
                }
            }
            catch (Exception Ex)
            {
                Error("Application exception: " + Ex.ToString());
                if (ReturnCode == 0)
                {
                    Program.ReturnCode = (int)ErrorCodes.Error_Unknown;
                }
            }

            Environment.ExitCode = ReturnCode;
            return (ReturnCode);
        }
    }
}

