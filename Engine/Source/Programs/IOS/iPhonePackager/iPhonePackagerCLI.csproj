<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <PublishSingleFile>true</PublishSingleFile>
    <Configuration Condition=" '$(Configuration)' == '' ">Development</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">AnyCPU</Platform>
    <OutputType>Exe</OutputType>
    <RootNamespace>IPhonePackagerCLI</RootNamespace>
    <SccProjectName>
    </SccProjectName>
    <SccLocalPath>
    </SccLocalPath>
    <SccAuxPath>
    </SccAuxPath>
    <SccProvider>
    </SccProvider>
    <IsWebBootstrapper>false</IsWebBootstrapper>
    <StartupObject>
    </StartupObject>
    <ApplicationIcon>IPhonePackagerIcon.ico</ApplicationIcon>
    <PublishUrl>publish\</PublishUrl>
    <Install>true</Install>
    <InstallFrom>Disk</InstallFrom>
    <UpdateEnabled>false</UpdateEnabled>
    <UpdateMode>Foreground</UpdateMode>
    <UpdateInterval>7</UpdateInterval>
    <UpdateIntervalUnits>Days</UpdateIntervalUnits>
    <UpdatePeriodically>false</UpdatePeriodically>
    <UpdateRequired>false</UpdateRequired>
    <MapFileExtensions>true</MapFileExtensions>
    <ApplicationRevision>0</ApplicationRevision>
    <ApplicationVersion>1.0.0.%2a</ApplicationVersion>
    <UseApplicationTrust>false</UseApplicationTrust>
    <BootstrapperEnabled>true</BootstrapperEnabled>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <UseWindowsForms>false</UseWindowsForms>
    <ImportWindowsDesktopTargets>false</ImportWindowsDesktopTargets>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">
    <OutputPath>..\..\..\..\Binaries\DotNET\IOS\</OutputPath>
    <PlatformTarget>AnyCPU</PlatformTarget>
    <DefineConstants>$(DefineConstants);IPP_CLI</DefineConstants>
  </PropertyGroup>
  <PropertyGroup Condition="'$(Configuration)|$(Platform)'=='Development|AnyCPU'">
    <DefineConstants>$(DefineConstants);IPP_CLI</DefineConstants>
    <Optimize>true</Optimize>
    <OutputPath>..\..\..\..\Binaries\DotNET\IOS\</OutputPath>
    <PlatformTarget>AnyCPU</PlatformTarget>
    <AppendTargetFrameworkToOutputPath>false</AppendTargetFrameworkToOutputPath>
    <AppendRuntimeIdentifierToOutputPath>false</AppendRuntimeIdentifierToOutputPath>
  </PropertyGroup>
  <PropertyGroup>
    <AppendTargetFrameworkToOutputPath>false</AppendTargetFrameworkToOutputPath>
    <AssemblyName>IPhonePackagerCLI</AssemblyName>
  </PropertyGroup>
  <ItemGroup>
    <Compile Remove="Resources\**" />
    <EmbeddedResource Remove="Resources\**" />
    <None Remove="Resources\**" />
  </ItemGroup>
  <ItemGroup>
    <Compile Remove="ConfigureMobileGame.cs" />
    <Compile Remove="ConfigureMobileGame.Designer.cs" />
    <Compile Remove="GenerateSigningRequestDialog.cs" />
    <Compile Remove="GenerateSigningRequestDialog.Designer.cs" />
    <Compile Remove="GraphicalResignTool.cs" />
    <Compile Remove="GraphicalResignTool.Designer.cs" />
    <Compile Remove="PasswordDialog.cs" />
    <Compile Remove="PasswordDialog.Designer.cs" />
    <Compile Remove="Properties\Resources.Designer.cs" />
    <Compile Remove="SlowProgressDialog.cs" />
    <Compile Remove="SlowProgressDialog.Designer.cs" />
    <Compile Remove="ToolsHub.Designer.cs" />
  </ItemGroup>
  <ItemGroup>
    <EmbeddedResource Remove="ConfigureMobileGame.resx" />
    <EmbeddedResource Remove="GenerateSigningRequestDialog.resx" />
    <EmbeddedResource Remove="GraphicalResignTool.resx" />
    <EmbeddedResource Remove="PasswordDialog.resx" />
    <EmbeddedResource Remove="Properties\Resources.resx" />
    <EmbeddedResource Remove="SlowProgressDialog.resx" />
    <EmbeddedResource Remove="ToolsHub.resx" />
  </ItemGroup>
  <ItemGroup>
    <None Remove="app.config" />
    <None Remove="IPhonePackagerIcon.ico" />
  </ItemGroup>
  <ItemGroup>
    <Reference Update="System.Core">
      <RequiredTargetFramework>3.5</RequiredTargetFramework>
    </Reference>
  </ItemGroup>
  <ItemGroup>
    <Compile Include="..\..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="System.Security.Cryptography.Pkcs" Version="10.0.6" />
  </ItemGroup>
  <ItemGroup>
    <Reference Include="BouncyCastle.Crypto">
      <HintPath>..\..\..\..\Binaries\ThirdParty\IOS\BouncyCastle.Crypto.dll</HintPath>
    </Reference>
    <Reference Include="Ionic.Zip.Reduced">
      <HintPath>..\..\..\..\Binaries\DotNET\Ionic.Zip.Reduced.dll</HintPath>
    </Reference>
    <Reference Include="SysadminsLV.Asn1Parser">
      <HintPath>..\..\..\..\Binaries\ThirdParty\IOS\SysadminsLV.Asn1Parser.dll</HintPath>
    </Reference>
  </ItemGroup>
</Project>